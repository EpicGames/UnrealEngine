{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Rigid Asset",
	"Description": "Rigid Asset plugin for creating and utilising collections of rigid bodies",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosRigidAssetEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		},
		{
			"Name": "ChaosRigidAssetNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosRigidAssetEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		}
	]
}
