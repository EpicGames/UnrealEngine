// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosRigidAssetEditor : ModuleRules
{
	public ChaosRigidAssetEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"UnrealEd",
				"SlateCore",
				"Slate",
				"DataflowCore",
				"DataflowEngine",
				"DataflowEnginePlugin",
				"DataflowEditor",
				"ChaosRigidAssetEngine",
				"ChaosRigidAssetNodes",
				"PhysicsAssetEditor"
			}
		);
		
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
			}
		);

		bAllowUETypesInNamespaces = true;

		SetupModulePhysicsSupport(Target);
	}
}
