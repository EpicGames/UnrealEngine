// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosRigidAssetEditorModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ContentBrowserModule.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowAttachment.h"
#include "DataflowRendering.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "PhysicsAssetDataflowAttachment.h"

IMPLEMENT_MODULE(FChaosRigidAssetEditorModule, ChaosRigidAssetEditor);

bool GRigidAssetModuleOverridePhysicsAssetEditor = false;
static FAutoConsoleVariableRef GRigidAssetModuleOverridePhysicsAssetEditorCVar(
	TEXT("p.rigidasset.enableeditoroverride"),
	GRigidAssetModuleOverridePhysicsAssetEditor,
	TEXT("Whether to enable the editor override for physics asset such that physics assets using dataflow will no longer open in the base physics asset editor, preferring the dataflow editor instead"));

/** 
 * Override for physics asset editor to instead route to the dataflow editor if there
 * is a dataflow attachment present in the asset.
 */
class FDataflowPhysicsAssetEditorOverride : public IPhysicsAssetEditorOverride
{
public:

	virtual ~FDataflowPhysicsAssetEditorOverride() = default;

	bool OpenAsset(UPhysicsAsset* InAsset) override;
};

UPhysicsAssetDataflowAttachment* GetDataflowAttachment(IInterface_AssetUserData* UserDataHolder)
{
	if(!UserDataHolder)
	{
		return nullptr;
	}

	return UserDataHolder->GetAssetUserData<UPhysicsAssetDataflowAttachment>();
}

void SpawnEditorFor(UPhysicsAsset* InAsset)
{
	UDataflowAttachment* Attachment = GetDataflowAttachment(InAsset);
	if(Attachment)
	{
		if(!Attachment->GetDataflowInstance().GetDataflowAsset())
		{
			Attachment->GetDataflowInstance().SetDataflowAsset(Cast<UDataflow>(UE::DataflowAssetDefinitionHelpers::NewOrOpenDataflowAsset(InAsset)));
		}

		if(!Attachment->GetDataflowInstance().GetDataflowAsset())
		{
			// User opted to not create a dataflow
			FNotificationInfo NotifyInfo(NSLOCTEXT("RigidAssetEditorModule", "NoDataflow", "No Dataflow asset available to edit."));
			NotifyInfo.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(NotifyInfo);
		}
		else
		{
			UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

			UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
			AssetEditor->RegisterToolCategories({ "General" });
			const TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr, TEXT("/ChaosRigidAsset/BP_PhysicsAssetPreview.BP_PhysicsAssetPreview_C"), nullptr, LOAD_None, nullptr);

			AssetEditor->Initialize({ GetDataflowAttachment(InAsset) }, ActorClass);
		}
	}
}

void FChaosRigidAssetEditorModule::StartupModule()
{
	EditorFeature = MakeUnique<FDataflowPhysicsAssetEditorOverride>();

	IModularFeatures::Get().RegisterModularFeature(IPhysicsAssetEditorOverride::ModularFeatureName, EditorFeature.Get());

	UE::Dataflow::FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<UE::Chaos::RigidAsset::FAggregateGeometryGeomRenderCallbacks>());
	UE::Dataflow::FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<UE::Chaos::RigidAsset::FBoneSelectionRenderCallbacks>());
	UE::Dataflow::FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<UE::Chaos::RigidAsset::FPhysAssetStateRenderCallbacks>());

	FDataflowAttachmentFactory::Get().Register(UPhysicsAsset::StaticClass()->GetFName(), [](UObject* Owner) { return NewObject<UPhysicsAssetDataflowAttachment>(Owner); });
	DataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(UPhysicsAsset::StaticClass());
}

void FChaosRigidAssetEditorModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IPhysicsAssetEditorOverride::ModularFeatureName, EditorFeature.Get());
	EditorFeature = nullptr;

	UE::Dataflow::FRenderingFactory::GetInstance()->DeregisterCallbacks(UE::Chaos::RigidAsset::FPhysAssetStateRenderCallbacks::StaticGetRenderKey());
	UE::Dataflow::FRenderingFactory::GetInstance()->DeregisterCallbacks(UE::Chaos::RigidAsset::FBoneSelectionRenderCallbacks::StaticGetRenderKey());
	UE::Dataflow::FRenderingFactory::GetInstance()->DeregisterCallbacks(UE::Chaos::RigidAsset::FAggregateGeometryGeomRenderCallbacks::StaticGetRenderKey());

	UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(DataflowMenusHandle);
}

bool FDataflowPhysicsAssetEditorOverride::OpenAsset(UPhysicsAsset* InAsset)
{
	if(!GRigidAssetModuleOverridePhysicsAssetEditor)
	{
		return false;
	}

	UDataflowAttachment* Attachment = GetDataflowAttachment(InAsset);
	if(Attachment)
	{
		SpawnEditorFor(InAsset);

		return true;
	}

	return false;
}
