// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering.h"

#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BoxElem.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"

#include "Engine/SkeletalMesh.h"
#include "ReferenceSkeleton.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "BoneSelection.h"
#include "PhysicsAssetDataflowState.h"

namespace UE::Chaos::RigidAsset
{
	void RenderAggGeom(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State, const FKAggregateGeom& AggGeom, const FTransform& BaseTm, FName BoneName, const FLinearColor& InColor)
	{
		const int32 NumElems = AggGeom.GetElementCount();

		if(NumElems == 0)
		{
			// Nothing to actually draw - return before opening a geometry group
			return;
		}

		for(int32 Index = 0; Index < NumElems; ++Index)
		{
			FString GeometryName = FString::Printf(TEXT("%s_%s_%d"), *State.GetGuid().ToString(), *BoneName.ToString(), Index);
			const int32 GeometryGroup = RenderingFacade.StartGeometryGroup(GeometryName);

			const FKShapeElem* Elem = AggGeom.GetElement(Index);

			if(!Elem)
			{
				break;
			}

			switch(Elem->GetShapeType())
			{
			case EAggCollisionShape::Box:
			{
				const FKBoxElem* AsBox = static_cast<const FKBoxElem*>(Elem);
				const FVector3f Extent = FVector3f{ AsBox->X, AsBox->Y, AsBox->Z } *0.5f;

				RenderingFacade.SetGroupTransform(GeometryGroup, Elem->GetTransform() * BaseTm);
				RenderingFacade.AddBox({ -Extent, Extent });

				break;
			}
			case EAggCollisionShape::Sphere:
			{
				const FKSphereElem* AsSphere = static_cast<const FKSphereElem*>(Elem);

				// Transform placed in the group - always draw at (0,0,0)
				RenderingFacade.SetGroupTransform(GeometryGroup, Elem->GetTransform() * BaseTm);
				RenderingFacade.AddSphere({0.0f, 0.0f, 0.0f}, AsSphere->Radius, InColor);

				break;
			}
			case EAggCollisionShape::Sphyl:
			{
				const FKSphylElem* AsSphyl = static_cast<const FKSphylElem*>(Elem);

				// AddCapsule uses the capsule mesh generator which has the origin of the capsule at one end of
				// the segment, we need to transform first to the centre of the capsule, then apply the elem transform
				const FTransform CapsuleCollisionAdjust(FQuat::Identity, FVector(0.0f, 0.0f, -AsSphyl->Length / 2));

				RenderingFacade.SetGroupTransform(GeometryGroup, CapsuleCollisionAdjust * Elem->GetTransform() * BaseTm);
				RenderingFacade.AddCapsule(AsSphyl->Length, AsSphyl->Radius);

				break;
			}
			case EAggCollisionShape::Convex:
			{
				const FKConvexElem* AsConvex = static_cast<const FKConvexElem*>(Elem);

				const int32 NumIndices = AsConvex->IndexData.Num();
				if(NumIndices == 0 || NumIndices % 3 != 0)
				{
					// Empty or invalid convex index data.
					break;
				}

				RenderingFacade.SetGroupTransform(GeometryGroup, Elem->GetTransform() * BaseTm);

				const int32 NumFaces = NumIndices / 3;
				TArray<FVector3f> Positions;
				Positions.Reserve(AsConvex->VertexData.Num());
				for(const FVector& Position : AsConvex->VertexData)
				{
					Positions.Add(static_cast<FVector3f>(Position));
				}

				TArray<FIntVector> Indices;
				Indices.Reserve(NumFaces);

				for(int32 FaceIndex = 0; FaceIndex < AsConvex->IndexData.Num(); FaceIndex += 3)
				{
					Indices.Add({
						AsConvex->IndexData[FaceIndex + 0],
						AsConvex->IndexData[FaceIndex + 2],
						AsConvex->IndexData[FaceIndex + 1]
					});
				}

				TArray<FLinearColor> Colors;
				Colors.SetNum(AsConvex->VertexData.Num());
				for(FLinearColor& Color : Colors)
				{
					Color = FLinearColor::White;
				}

				RenderingFacade.AddFaces(Positions, Indices, Colors);

				break;
			}
			default:
				break;
			}

			RenderingFacade.EndGeometryGroup(GeometryGroup);
		}
	}

	UE::Dataflow::FRenderKey FAggregateGeometryGeomRenderCallbacks::StaticGetRenderKey()
	{
		return { "GeomRender", FName("FKAggregateGeom") };
	}

	UE::Dataflow::FRenderKey FAggregateGeometryGeomRenderCallbacks::GetRenderKey() const
	{
		return StaticGetRenderKey();
	}

	bool FAggregateGeometryGeomRenderCallbacks::CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const
	{
		return true;
	}

	void FAggregateGeometryGeomRenderCallbacks::Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State)
	{
		if(State.GetRenderOutputs().Num() > 0)
		{

			FKAggregateGeom Default;
			const FKAggregateGeom& AggGeom = State.GetValue<FKAggregateGeom>(State.GetRenderOutputs()[0], Default);

			RenderAggGeom(RenderingFacade, State, AggGeom, FTransform::Identity, NAME_None, FLinearColor::White);
		}
	}

	UE::Dataflow::FRenderKey FBoneSelectionRenderCallbacks::StaticGetRenderKey()
	{
		return { "GeomRender", FName("FBoneSelection") };
	}

	UE::Dataflow::FRenderKey FBoneSelectionRenderCallbacks::GetRenderKey() const
	{
		return StaticGetRenderKey();
	}

	bool FBoneSelectionRenderCallbacks::CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const
	{
		return true;
	}

	// GeometryCore perp vectors impl, not currently exposed in that module
	template <typename RealType>
	void MakePerpVectors(const UE::Math::TVector<RealType>& Normal, UE::Math::TVector<RealType>& OutPerp1, UE::Math::TVector<RealType>& OutPerp2)
	{
		// Duff et al method, from https://graphics.pixar.com/library/OrthonormalB/paper.pdf
		if(Normal.Z < (RealType)0)
		{
			RealType A = (RealType)1 / ((RealType)1 - Normal.Z);
			RealType B = Normal.X * Normal.Y * A;
			OutPerp1.X = (RealType)1 - Normal.X * Normal.X * A;
			OutPerp1.Y = -B;
			OutPerp1.Z = Normal.X;
			OutPerp2.X = B;
			OutPerp2.Y = Normal.Y * Normal.Y * A - (RealType)1;
			OutPerp2.Z = -Normal.Y;
		}
		else
		{
			RealType A = (RealType)1 / ((RealType)1 + Normal.Z);
			RealType B = -Normal.X * Normal.Y * A;
			OutPerp1.X = (RealType)1 - Normal.X * Normal.X * A;
			OutPerp1.Y = B;
			OutPerp1.Z = -Normal.X;
			OutPerp2.X = B;
			OutPerp2.Y = (RealType)1 - Normal.Y * Normal.Y * A;
			OutPerp2.Z = -Normal.Y;
		}
	}

	void DrawBone(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const FVector3f& From, const FVector3f& To, const FLinearColor& InColor)
	{
		float Length;
		FVector3f Axes[3];
		const FVector3f Segment = To - From;
		Segment.ToDirectionAndLength(Axes[2], Length);
		MakePerpVectors(Axes[2], Axes[0], Axes[1]);

		constexpr float WidePoint = 0.3f;
		constexpr float Thickness = 2.5f;

		TArray<FVector3f> Verts;
		TArray<FLinearColor> Colors;
		Verts.SetNumUninitialized(6);
		Colors.SetNumUninitialized(6);

		Verts[0] = From;
		Verts[5] = To;

		Verts[1] = From + (Axes[0] + Axes[1]) * Thickness + Axes[2] * (WidePoint * Length);
		Verts[2] = From + (Axes[0] + -Axes[1]) * Thickness + Axes[2] * (WidePoint * Length);
		Verts[3] = From + (-Axes[0] + -Axes[1]) * Thickness + Axes[2] * (WidePoint * Length);
		Verts[4] = From + (-Axes[0] + Axes[1]) * Thickness + Axes[2] * (WidePoint * Length);

		TArray<FIntVector> Indices;
		Indices.SetNumUninitialized(8);

		// Top Section
		Indices[0] = FIntVector{ 0, 2, 1 };
		Indices[1] = FIntVector{ 0, 3, 2 };
		Indices[2] = FIntVector{ 0, 4, 3 };
		Indices[3] = FIntVector{ 0, 1, 4 };

		// Bottom Section
		Indices[4] = FIntVector{ 5, 1, 2 };
		Indices[5] = FIntVector{ 5, 2, 3 };
		Indices[6] = FIntVector{ 5, 3, 4 };
		Indices[7] = FIntVector{ 5, 4, 1 };

		for(FLinearColor& Color : Colors)
		{
			Color = InColor;
		}

		RenderingFacade.AddFaces(Verts, Indices, Colors);
	}

	void FBoneSelectionRenderCallbacks::Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State)
	{
		if(State.GetRenderOutputs().Num() > 0)
		{
			FRigidAssetBoneSelection Default;
			const FRigidAssetBoneSelection& Selection = State.GetValue<FRigidAssetBoneSelection>(State.GetRenderOutputs()[0], Default);

			if(!Selection.Mesh || !Selection.Skeleton)
			{
				return;
			}

			const FReferenceSkeleton& RefSkel = Selection.Mesh->GetRefSkeleton();

			TArray<FTransform> BoneTransforms;
			TArray<int32> ParentIndices;


			enum class ELocalSelectionType
			{
				None = 0,
				Selected,
				Merged
			};

			TArray<ELocalSelectionType> SelectedIndices;

			RefSkel.GetBoneAbsoluteTransforms(BoneTransforms);

			const int32 NumBones = BoneTransforms.Num();

			SelectedIndices.SetNumZeroed(NumBones);
			for(const FRigidAssetBoneInfo& Selected : Selection.SelectedBones)
			{
				SelectedIndices[Selected.Index] = ELocalSelectionType::Selected;

				for(int32 MergeIndex : Selected.MergedChildren)
				{
					SelectedIndices[MergeIndex] = ELocalSelectionType::Merged;
				}
			}

			ParentIndices.SetNum(NumBones);
			for(int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				ParentIndices[BoneIndex] = RefSkel.GetParentIndex(BoneIndex);
			}

			int32 Group = RenderingFacade.StartGeometryGroup(TEXT("BoneSelection"));
			for(int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FTransform& BoneTransform = BoneTransforms[BoneIndex];

				RenderingFacade.AddPoint((FVector3f)BoneTransform.GetTranslation());

				const int32 ParentIndex = ParentIndices[BoneIndex];
				if(ParentIndex != INDEX_NONE)
				{
					FLinearColor BoneColor = FLinearColor::Gray;

					switch(SelectedIndices[ParentIndex])
					{
					case ELocalSelectionType::Selected:
						BoneColor = FLinearColor(1.0f, 0.6f, 0.0f);
						break;
					case ELocalSelectionType::Merged:
						BoneColor = FLinearColor(0.0f, 0.8f, 0.0f);
						break;
					default:
						break;
					}

					DrawBone(RenderingFacade, (FVector3f)BoneTransforms[ParentIndex].GetTranslation(), (FVector3f)BoneTransforms[BoneIndex].GetTranslation(), BoneColor);
				}
			}
			RenderingFacade.EndGeometryGroup(Group);
		}
	}

	UE::Dataflow::FRenderKey FPhysAssetStateRenderCallbacks::StaticGetRenderKey()
	{
		return { "GeomRender", FName("FPhysicsAssetDataflowState") };
	}

	UE::Dataflow::FRenderKey FPhysAssetStateRenderCallbacks::GetRenderKey() const
	{
		return StaticGetRenderKey();
	}

	bool FPhysAssetStateRenderCallbacks::CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const
	{
		return true;
	}

	void FPhysAssetStateRenderCallbacks::Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State)
	{
		if(State.GetRenderOutputs().Num() > 0)
		{
			FPhysicsAssetDataflowState Default;
			const FPhysicsAssetDataflowState& AssetState = State.GetValue<FPhysicsAssetDataflowState>(State.GetRenderOutputs()[0], Default);

			if(AssetState.HasData())
			{
				const FReferenceSkeleton& RefSkeleton = AssetState.GetMesh()->GetRefSkeleton();

				for(TObjectPtr<const USkeletalBodySetup> Body : AssetState.GetBodies())
				{
					const int32 BoneIndex = RefSkeleton.FindBoneIndex(Body->BoneName);

					if(BoneIndex != INDEX_NONE)
					{
						FTransform BoneTransform = RefSkeleton.GetBoneAbsoluteTransform(BoneIndex);
						RenderAggGeom(RenderingFacade, State, Body->AggGeom, BoneTransform, Body->BoneName, FLinearColor(1.0f, 1.0f, 1.0f, 0.5f));
					}
				}
			}
		}
	}
}
