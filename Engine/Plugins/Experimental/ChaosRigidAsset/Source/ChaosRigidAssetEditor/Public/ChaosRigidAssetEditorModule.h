// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "IPhysicsAssetEditor.h"
#include "Delegates/IDelegateInstance.h"

class FDataflowPhysicsAssetEditorOverride;

class FChaosRigidAssetEditorModule : public IModuleInterface
{
public:

	CHAOSRIGIDASSETEDITOR_API virtual void StartupModule() override;
	CHAOSRIGIDASSETEDITOR_API virtual void ShutdownModule() override;

private:
	TUniquePtr<FDataflowPhysicsAssetEditorOverride> EditorFeature;
	FDelegateHandle AssetMenuExtenderHandle;
	FDelegateHandle DataflowMenusHandle;
};
