// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowRenderingFactory.h"

namespace UE::Chaos::RigidAsset
{
	class FAggregateGeometryGeomRenderCallbacks : public UE::Dataflow::FRenderingFactory::ICallbackInterface
	{
	public:

		static UE::Dataflow::FRenderKey StaticGetRenderKey();

		UE::Dataflow::FRenderKey GetRenderKey() const override;
		bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override;
		void Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State) override;

	};

	class FBoneSelectionRenderCallbacks : public UE::Dataflow::FRenderingFactory::ICallbackInterface
	{
	public:

		static UE::Dataflow::FRenderKey StaticGetRenderKey();

		UE::Dataflow::FRenderKey GetRenderKey() const override;
		bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override;
		void Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State) override;
	};

	class FPhysAssetStateRenderCallbacks : public UE::Dataflow::FRenderingFactory::ICallbackInterface
	{
	public:

		static UE::Dataflow::FRenderKey StaticGetRenderKey();

		UE::Dataflow::FRenderKey GetRenderKey() const override;
		bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override;
		void Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UE::Dataflow::FGraphRenderingState& State) override;
	};
}
