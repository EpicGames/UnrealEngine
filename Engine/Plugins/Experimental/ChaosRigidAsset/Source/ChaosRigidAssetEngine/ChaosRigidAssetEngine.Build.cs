// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosRigidAssetEngine : ModuleRules
{
	public ChaosRigidAssetEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"DataflowCore",
				"DataflowEngine",
				"DataflowEnginePlugin"
			}
		);
		
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
			}
		);

		bAllowUETypesInNamespaces = true;

		SetupModulePhysicsSupport(Target);
	}
}
