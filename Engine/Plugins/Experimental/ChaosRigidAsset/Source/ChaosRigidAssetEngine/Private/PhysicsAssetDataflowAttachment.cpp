// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetDataflowAttachment.h"
#include "PhysicsAssetDataflowContent.h"
#include "PhysicsEngine/PhysicsAsset.h"


void UPhysicsAssetDataflowAttachment::WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const
{
	if(const TObjectPtr<UPhysicsAssetDataflowContent> Content = Cast<UPhysicsAssetDataflowContent>(DataflowContent))
	{
		Content->SetDataflowAsset(GetDataflowInstance().GetDataflowAsset());
		Content->SetDataflowTerminal(GetDataflowInstance().GetDataflowTerminal().ToString());
	
		if(UPhysicsAsset* PhysAsset = GetTypedOuter<UPhysicsAsset>())
		{
			Content->SetSkeletalMesh(PhysAsset->GetPreviewMesh());
			Content->SetPhysicsAsset(PhysAsset);
		}
	}
}

void UPhysicsAssetDataflowAttachment::ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent)
{

}

TObjectPtr<UDataflowBaseContent> UPhysicsAssetDataflowAttachment::CreateDataflowContent()
{
	TObjectPtr<UDataflowBaseContent> BaseContent = NewObject<UPhysicsAssetDataflowContent>();
	
	BaseContent->SetDataflowOwner(this);
	BaseContent->SetTerminalAsset(this);
	
	WriteDataflowContent(BaseContent);
	
	return BaseContent;
}

FString UPhysicsAssetDataflowAttachment::GetPreviewActorPath() const
{
	return TEXT("/ChaosRigidAsset/BP_PhysicsAssetPreview.BP_PhysicsAssetPreview_C");
}

