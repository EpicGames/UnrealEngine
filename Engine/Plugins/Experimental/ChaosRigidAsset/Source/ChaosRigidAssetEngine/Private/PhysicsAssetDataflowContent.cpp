// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetDataflowContent.h"
#include "Engine/SkeletalMesh.h"
#include "PhysicsEngine/PhysicsAsset.h"

void UPhysicsAssetDataflowContent::SetActorProperties(TObjectPtr<AActor>& PreviewActor) const
{
	Super::SetActorProperties(PreviewActor);
	OverrideActorProperty(PreviewActor, SkelMesh, TEXT("PreviewMesh"));
	OverrideActorProperty(PreviewActor, PhysAsset, TEXT("PhysicsAsset"));
}

void UPhysicsAssetDataflowContent::SetSkeletalMesh(USkeletalMesh* InMesh)
{
	SkelMesh = InMesh;
}

void UPhysicsAssetDataflowContent::SetPhysicsAsset(UPhysicsAsset* InAsset)
{
	PhysAsset = InAsset;
}
