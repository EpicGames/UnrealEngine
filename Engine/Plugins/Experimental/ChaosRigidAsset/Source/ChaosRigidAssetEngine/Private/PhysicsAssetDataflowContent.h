// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Dataflow/DataflowContent.h"

#include "PhysicsAssetDataflowContent.generated.h"

class AActor;
class USkeletalMesh;
class UPhysicsAsset;

/**
 * Dataflow editor content for physics assets
 */
UCLASS()
class UPhysicsAssetDataflowContent : public UDataflowBaseContent
{
	GENERATED_BODY()

public:

	void SetActorProperties(TObjectPtr<AActor>& PreviewActor) const override;

	void SetSkeletalMesh(USkeletalMesh* InMesh);
	void SetPhysicsAsset(UPhysicsAsset* InAsset);

private:
	
	// Preview mesh to use
	UPROPERTY()
	TObjectPtr<USkeletalMesh> SkelMesh = nullptr;

	// Physics asset to use (must be compatible with the preview mesh)
	UPROPERTY()
	TObjectPtr<UPhysicsAsset> PhysAsset = nullptr;

};