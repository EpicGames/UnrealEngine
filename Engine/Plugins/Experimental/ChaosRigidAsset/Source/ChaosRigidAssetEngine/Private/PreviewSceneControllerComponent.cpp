// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewSceneControllerComponent.h"

#include "Chaos/ChaosScene.h"
#include "ChaosSolverConfiguration.h"
#include "Engine/World.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"

void UPreviewSceneControllerComponent::ApplySolverConfig(const FChaosSolverConfiguration& Config)
{
	if(UWorld* World = GetWorld())
	{
		if(FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			PhysScene->GetSolver()->ApplyConfig(Config);
		}
	}
}

void UPreviewSceneControllerComponent::EnableAsyncTick(float InAsyncDt)
{
	if(UWorld* World = GetWorld())
	{
		if(FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			PhysScene->GetSolver()->EnableAsyncMode(InAsyncDt);
		}
	}
}

void UPreviewSceneControllerComponent::DisableAsyncTick()
{
	if(UWorld* World = GetWorld())
	{
		if(FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			PhysScene->GetSolver()->DisableAsyncMode();
		}
	}
}
