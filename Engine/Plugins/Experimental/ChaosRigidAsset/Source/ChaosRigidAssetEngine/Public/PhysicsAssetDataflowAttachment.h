// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowAttachment.h"
#include "PhysicsAssetDataflowAttachment.generated.h"

UCLASS()
class UPhysicsAssetDataflowAttachment : public UDataflowAttachment
{
	GENERATED_BODY()

public:

	// IDataflowContentOwner
	void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const override;
	void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) override;
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

protected:

	// IDataflowContentOwner
	TObjectPtr<UDataflowBaseContent> CreateDataflowContent() override;
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	// UDataflowAttachment
	FString GetPreviewActorPath() const override;
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

private:
};
