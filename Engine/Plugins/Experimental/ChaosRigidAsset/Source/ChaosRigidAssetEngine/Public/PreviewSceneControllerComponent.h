// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "PreviewSceneControllerComponent.generated.h"

#define UE_API CHAOSRIGIDASSETENGINE_API

struct FChaosSolverConfiguration;

/**
 * Controller for simulation solver interaction in a dataflow preview scene
 */
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class UPreviewSceneControllerComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category= PreviewSceneControl)
	UE_API void ApplySolverConfig(const FChaosSolverConfiguration& Config);

	UFUNCTION(BlueprintCallable, Category= PreviewSceneControl)
	UE_API void EnableAsyncTick(float InAsyncDt);

	UFUNCTION(BlueprintCallable, Category= PreviewSceneControl)
	UE_API void DisableAsyncTick();

private:
};

#undef UE_API