// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CopyOnWriteArray.h"

#include "PhysicsAssetDataflowState.generated.h"

class USkeleton;
class USkeletalMesh;
class USkeletalBodySetup;
class UPhysicsConstraintTemplate;

class FReferenceCollector;

/**
 * State container for physics asset data during dataflow graph evaluation.
 * Represents the complete state required to build a physics asset. Datflow nodes populate this container
 * until it reaches a terminal which packages it into a final UPhysicsAsset. This allows us to keep the
 * const invariant for dataflow evaluation by adding copy-on-write support to the state.
 * During node evaluation this object should be passed by-value as it is a collection of pts to the
 * actual data. If the copy-on-write arrays are modified they will automatically duplicate if required
 */
USTRUCT()
struct FPhysicsAssetDataflowState
{
	GENERATED_BODY()

public:

	CHAOSRIGIDASSETNODES_API FPhysicsAssetDataflowState();
	explicit CHAOSRIGIDASSETNODES_API FPhysicsAssetDataflowState(TObjectPtr<USkeleton> TargetSkeleton, TObjectPtr<USkeletalMesh> TargetMesh);

	CHAOSRIGIDASSETNODES_API USkeleton* GetSkeleton() const;
	CHAOSRIGIDASSETNODES_API USkeletalMesh* GetMesh() const;
	CHAOSRIGIDASSETNODES_API USkeletalBodySetup* GetBody(FName BoneName) const;
	CHAOSRIGIDASSETNODES_API void SetBody(FName BoneName, USkeletalBodySetup* InSetup);
	CHAOSRIGIDASSETNODES_API USkeletalBodySetup* FindOrCreateBody(FName BoneName, USkeletalBodySetup* TemplateBody = nullptr);
	CHAOSRIGIDASSETNODES_API USkeletalBodySetup* FindOrCreateBodyChecked(FName BoneName, USkeletalBodySetup* TemplateBody = nullptr);
	CHAOSRIGIDASSETNODES_API const TCopyOnWriteArray<TObjectPtr<USkeletalBodySetup>>& GetBodies() const;

	void AddStructReferencedObjects(FReferenceCollector& Collector);

	// Bodies and constraints in the simulation setup
	TCopyOnWriteArray<TObjectPtr<UPhysicsConstraintTemplate>> Constraints;

	void DebugLog() const;
	CHAOSRIGIDASSETNODES_API bool HasData() const;

private:

	// Base asset data
	TObjectPtr<USkeleton> TargetSkeleton = nullptr;
	TObjectPtr<USkeletalMesh> TargetMesh = nullptr;

	TCopyOnWriteArray<int32> BoneIndexToBody;
	TCopyOnWriteArray<TObjectPtr<USkeletalBodySetup>> Bodies;
};

/**
 * The state struct has UObject references within the copy-on-write arrays, enable handling for that here
 */
template<>
struct TStructOpsTypeTraits<FPhysicsAssetDataflowState> : public TStructOpsTypeTraitsBase2<FPhysicsAssetDataflowState>
{
	enum
	{
		WithAddStructReferencedObjects = true
	};
};