// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimationTerminalNodes.h"

#include "Animation/AnimBank.h"
#include "Animation/AnimSequence.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"


void UE::Dataflow::RegisterAnimationTerminalNodes()
{
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAnimBankAssetTerminalNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAnimSequenceAssetTerminalNode);
}


////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowAnimSequenceAssetTerminalNode::FDataflowAnimSequenceAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Animation);
	RegisterInputConnection(&AnimationAsset);
}

void FDataflowAnimSequenceAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	UAnimSequence* AnimTarget = Asset ? Cast<UAnimSequence>(Asset.Get()) : nullptr;
	if (!AnimTarget)
	{
		AnimTarget = GetValue(Context, &AnimationAsset);
	}

	if (AnimTarget)
	{
#if WITH_EDITOR
		// This should be const but CreateAnimation does not take a const parameter
		if (UAnimSequence* InAnimation = GetValue(Context, &Animation))
		{
			// copied from UAnimSequence::CreateAnimation because the method does not take a const anim sequence as a parameter
			AnimTarget->CreateAnimation(InAnimation);
		}

		AnimTarget->Modify();
		AnimTarget->PostEditChange();
#endif // WITH_EDITOR
	}
}

void FDataflowAnimSequenceAssetTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
}


////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowAnimBankAssetTerminalNode::FDataflowAnimBankAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&AnimationBankAsset);

	for (int32 Index = 0; Index < NumInitialInputs; ++Index)
	{
		AddPins();
	}
}

void FDataflowAnimBankAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	UAnimBank* AnimBankTarget = Asset ? Cast<UAnimBank>(Asset.Get()) : nullptr;
	if (!AnimBankTarget)
	{
		AnimBankTarget = GetValue(Context, &AnimationBankAsset);
	}

	if (AnimBankTarget)
	{
#if WITH_EDITORONLY_DATA
		AnimBankTarget->Sequences.Reset();
		for (int32 Index = 0; Index < AnimSequences.Num(); ++Index)
		{
			TObjectPtr<UAnimSequence> AnimSequence = GetValue(Context, GetConnectionReference(Index), TObjectPtr<UAnimSequence>(nullptr));
			if (AnimSequence)
			{
				FAnimBankSequence& AnimBankSequence = AnimBankTarget->Sequences.AddDefaulted_GetRef();
				AnimBankSequence.Sequence = AnimSequence;
				AnimBankSequence.bAutoStart = bAutoStart;
				AnimBankSequence.bLooping = bLooping;
				AnimBankSequence.Position = Position;
				AnimBankSequence.PlayRate = PlayRate;
				AnimBankSequence.BoundsScale = BoundsScale;
			}
		}
		AnimBankTarget->Modify();
		AnimBankTarget->PostEditChange();
#endif // WITH_EDITORONLY_DATA
	}
}

void FDataflowAnimBankAssetTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
}

bool FDataflowAnimBankAssetTerminalNode::CanAddPin() const 
{ 
	return true; 
}

TArray<UE::Dataflow::FPin> FDataflowAnimBankAssetTerminalNode::AddPins()
{
	const int32 Index = AnimSequences.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

bool FDataflowAnimBankAssetTerminalNode::CanRemovePin() const 
{ 
	return AnimSequences.Num() > NumInitialInputs;
}

TArray<UE::Dataflow::FPin> FDataflowAnimBankAssetTerminalNode::GetPinsToRemove() const
{
	if (AnimSequences.Num() > 0)
	{
		const int32 Index = AnimSequences.Num() - 1;
		if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
		{
			return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
		}
	}
	return Super::GetPinsToRemove();
}

void FDataflowAnimBankAssetTerminalNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	if (AnimSequences.Num() > 0)
	{
		const int32 Index = AnimSequences.Num() - 1;
		check(AnimSequences.IsValidIndex(Index));
#if DO_CHECK
		const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
		check(Input);
		check(Input->GetName() == Pin.Name);
		check(Input->GetType() == Pin.Type);
#endif
		AnimSequences.SetNum(Index);
	}
	return Super::OnPinRemoved(Pin);
}

void FDataflowAnimBankAssetTerminalNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// Make sure we have the right number of initial inputs
		for (int32 Index = AnimSequences.Num(); Index < NumInitialInputs; ++Index)
		{
			AddPins();
		}
		for (int32 Index = 0; Index < NumInitialInputs; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialInputs; Index < AnimSequences.Num(); ++Index)
		{
			FDataflowInput& Input = FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}

		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs() - NumRequiredDataflowInputs;
			const int32 OrigNumInputs = AnimSequences.Num();
			if (OrigNumRegisteredInputs > OrigNumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Inputs so we can get connection references.
				AnimSequences.SetNum(OrigNumRegisteredInputs);
				for (int32 Index = OrigNumInputs; Index < AnimSequences.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				AnimSequences.SetNum(OrigNumInputs);
			}
		}
		else
		{
			// Index + all Inputs
			ensureAlways(AnimSequences.Num() + NumRequiredDataflowInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<TObjectPtr<UAnimSequence>> FDataflowAnimBankAssetTerminalNode::GetConnectionReference(int32 Index) const
{
	return { &AnimSequences[Index], Index, &AnimSequences };
}
