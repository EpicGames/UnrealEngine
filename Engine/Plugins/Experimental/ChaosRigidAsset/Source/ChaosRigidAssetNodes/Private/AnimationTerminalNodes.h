// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowTerminalNode.h"

#include "AnimationTerminalNodes.generated.h"

class UAnimSequence;
class UAnimBank;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Terminal node that writes to a animation sequence asset
 */
USTRUCT(meta = (DataflowTerminal))
struct FDataflowAnimSequenceAssetTerminalNode : public FDataflowTerminalNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAnimSequenceAssetTerminalNode, "AnimSequenceTerminal", "Terminal", "Animation Keys")

public:

	FDataflowAnimSequenceAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UAnimSequence> Animation;

	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<UAnimSequence> AnimationAsset;

	void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	void Evaluate(UE::Dataflow::FContext& Context) const override;
};


/**
 * Terminal node that writes to a animation bank asset
 * an AnimBank is made of multiple anim sequences and used to animate GPU 
 */
USTRUCT(meta = (DataflowTerminal))
struct FDataflowAnimBankAssetTerminalNode : public FDataflowTerminalNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAnimBankAssetTerminalNode, "AnimBankTerminal", "Terminal", "Animation Keys")

public:
	FDataflowAnimBankAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<UAnimBank> AnimationBankAsset = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UAnimSequence>> AnimSequences;

	UPROPERTY(EditAnywhere, Category = Animation)
	bool bLooping = true;

	UPROPERTY(EditAnywhere, Category = Animation)
	bool bAutoStart = true;

	UPROPERTY(EditAnywhere, Category = Animation)
	float Position = 0.0f;

	UPROPERTY(EditAnywhere, Category = Animation)
	float PlayRate = 1.0f;

	/**
	 * Scales the bounds of the instances playing this sequence.
	 * This is useful when the animation moves the vertices of the mesh outside of its bounds.
	 * Warning: Increasing the bounds will reduce performance!
	 */
	UPROPERTY(EditAnywhere, Category = Animation, meta = (ClampMin = "1", UIMin = "1", ClampMax = "10.0", UIMax = "10.0"))
	float BoundsScale = 1.0f;

	void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	void Evaluate(UE::Dataflow::FContext& Context) const override;

	/* Begin FDataflowNode interface */
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	/* End FDataflowNode interface */

	UE::Dataflow::TConnectionReference<TObjectPtr<UAnimSequence>> GetConnectionReference(int32 Index) const;

	inline static constexpr int32 NumRequiredDataflowInputs = 1;
	inline static constexpr int32 NumInitialInputs = 2;

};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterAnimationTerminalNodes();
}
