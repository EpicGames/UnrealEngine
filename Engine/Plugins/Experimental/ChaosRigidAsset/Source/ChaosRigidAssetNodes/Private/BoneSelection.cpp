// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneSelection.h"

FRigidAssetBoneInfo::FRigidAssetBoneInfo(FName InName, int32 InIndex, int32 InDepth) : Name(InName)
, Index(InIndex)
, Depth(InDepth)
{

}

bool FRigidAssetBoneInfo::operator<(const FRigidAssetBoneInfo& Other) const
{
	return Depth < Other.Depth;
}

bool FRigidAssetBoneSelection::ContainsIndex(int32 InBoneIndex) const
{
	const auto FindByIndex = [&InBoneIndex](const FRigidAssetBoneInfo& Item)
		{
			return Item.Index == InBoneIndex;
		};

	return SelectedBones.FindByPredicate(FindByIndex) != nullptr;
}

bool FRigidAssetBoneSelection::Contains(FName InBoneName) const
{
	const auto FindByName = [&InBoneName](const FRigidAssetBoneInfo& Item)
		{
			return Item.Name == InBoneName;
		};

	return SelectedBones.FindByPredicate(FindByName) != nullptr;
}

void FRigidAssetBoneSelection::SortBones()
{
	SelectedBones.Sort();
}
