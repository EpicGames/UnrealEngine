// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosRigidAssetNodesModule.h"
#include "ShapeElemNodes.h"
#include "PhysicsAssetDataflowNodes.h"
#include "AnimationTerminalNodes.h"
#include "Generators/SimulationAffector.h"

IMPLEMENT_MODULE(FChaosRigidAssetNodesModule, ChaosRigidAssetNodes);

 void FChaosRigidAssetNodesModule::StartupModule()
{
	UE::Dataflow::RegisterSimulationAffectorNodes();
	UE::Dataflow::RegisterPhysicsAssetTerminalNode();
	UE::Dataflow::RegisterPhysicsAssetNodes();
	UE::Dataflow::RegisterShapeNodes();
	UE::Dataflow::RegisterAnimationTerminalNodes();
 }

 void FChaosRigidAssetNodesModule::ShutdownModule()
{

}
