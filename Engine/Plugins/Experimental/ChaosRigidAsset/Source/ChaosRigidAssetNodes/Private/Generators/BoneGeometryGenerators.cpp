// Copyright Epic Games, Inc. All Rights Reserved.


#include "Generators/BoneGeometryGenerators.h"

#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Engine/SkeletalMesh.h"
#include "Math/Box.h"
#include "PhysicsEngine/ConvexElem.h"
#include "ReferenceSkeleton.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "Operations/MeshConvexHull.h"

DEFINE_LOG_CATEGORY(LogBoneGeometryGenerators);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static bool GBoneGeometryGeneratorsRunParallel = true;
static FAutoConsoleVariableRef CvarBoneGeometryGeneratorsRunParallel(
	TEXT("p.dataflow.bonegeometrygenerator.parallel"),
	GBoneGeometryGeneratorsRunParallel,
	TEXT("Whether bone geometry generator algorithms can run in parallel"));

template<int32 SerialThreshold = 4, typename Callable>
TArray<UE::Tasks::FTask> DispatchTasks(const TCHAR* InDebugName, int32 NumTasks, Callable&& InCallable)
{
	const bool bRunParallel = NumTasks > SerialThreshold && GBoneGeometryGeneratorsRunParallel && FApp::ShouldUseThreadingForPerformance();

	if(!bRunParallel)
	{
		for(int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			InCallable(TaskIndex);
		}

		// No tasks returned, didn't go wide
		return {};
	}
	else
	{
		TArray<UE::Tasks::FTask> Tasks;
		Tasks.AddDefaulted(NumTasks);

		for(int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			Tasks[TaskIndex] = UE::Tasks::Launch(InDebugName, [TaskIndex, &InCallable]() { InCallable(TaskIndex); }, LowLevelTasks::ETaskPriority::High);
		}

		return Tasks;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterBoneGeometryGeneratorNodes()
	{
		// Body Generators
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeBoxBoneGeometryGenerator);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSphereBoneGeometryGenerator);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCapsuleBoneGeometryGenerator);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeConvexBoneGeometryGenerator);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeConvexDecompBoneGeometryGenerator);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMakeBoxBoneGeometryGenerator::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Generator))
	{
		TObjectPtr<UBoneGeometryGenerator> OutGenerator = CastChecked<UBoneGeometryGenerator>(StaticDuplicateObject(Generator, GetTransientPackage()));
		SetValue(Context, OutGenerator, &Generator);
	}
}

void FMakeBoxBoneGeometryGenerator::Register()
{
	AddOutputs(Generator);

	Generator = NewObject<UBoneGeometryGenerator_Box>(GetOwner());
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMakeSphereBoneGeometryGenerator::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Generator))
	{
		TObjectPtr<UBoneGeometryGenerator> OutGenerator = CastChecked<UBoneGeometryGenerator>(StaticDuplicateObject(Generator, GetTransientPackage()));
		SetValue(Context, OutGenerator, &Generator);
	}
}

void FMakeSphereBoneGeometryGenerator::Register()
{
	AddOutputs(Generator);

	Generator = NewObject<UBoneGeometryGenerator_Sphere>(GetOwner());
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMakeCapsuleBoneGeometryGenerator::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Generator))
	{
		TObjectPtr<UBoneGeometryGenerator> OutGenerator = CastChecked<UBoneGeometryGenerator>(StaticDuplicateObject(Generator, GetTransientPackage()));
		SetValue(Context, OutGenerator, &Generator);
	}
}

void FMakeCapsuleBoneGeometryGenerator::Register()
{
	AddOutputs(Generator);

	Generator = NewObject<UBoneGeometryGenerator_Capsule>(GetOwner());
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMakeConvexBoneGeometryGenerator::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Generator))
	{
		TObjectPtr<UBoneGeometryGenerator> OutGenerator = CastChecked<UBoneGeometryGenerator>(StaticDuplicateObject(Generator, GetTransientPackage()));
		SetValue(Context, OutGenerator, &Generator);
	}
}

void FMakeConvexBoneGeometryGenerator::Register()
{
	AddOutputs(Generator);

	Generator = NewObject<UBoneGeometryGenerator_Convex>(GetOwner());
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMakeConvexDecompBoneGeometryGenerator::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Generator))
	{
		TObjectPtr<UBoneGeometryGenerator> OutGenerator = CastChecked<UBoneGeometryGenerator>(StaticDuplicateObject(Generator, GetTransientPackage()));
		SetValue(Context, OutGenerator, &Generator);
	}
}

void FMakeConvexDecompBoneGeometryGenerator::Register()
{
	AddOutputs(Generator);

	Generator = NewObject<UBoneGeometryGenerator_ConvexDecomposition>(GetOwner());
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
bool UBoneGeometryGenerator::CanDebugDraw() const
{
	return false;
}

bool UBoneGeometryGenerator::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return false;
}

void UBoneGeometryGenerator::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowNode::FDebugDrawParameters& DebugDrawParameters, FRigidAssetBoneSelection Bones) const
{
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FBoneVertData GetBoneVertexData(FName BoneName, USkeletalMesh* Mesh, int32 Lod, EVertexSelectMode SelectMode)
{
	if(!Mesh)
	{
		return {};
	}

	const FSkeletalMeshRenderData* MeshRenderData = Mesh->GetResourceForRendering();
	if(!MeshRenderData || !MeshRenderData->LODRenderData.IsValidIndex(Lod))
	{
		return {};
	}

	FBoneVertData Result;

	const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	const FSkeletalMeshLODRenderData& LodData = MeshRenderData->LODRenderData[Lod];

	constexpr float WeightsToFloatMult = (1.0f / 65535.0f);
	const FSkinWeightVertexBuffer* Weights = LodData.GetSkinWeightVertexBuffer();
	check(Weights);

	// Mesh buffer for normals, position buffer for positions
	const FStaticMeshVertexBuffer& MeshVertexBuffer = LodData.StaticVertexBuffers.StaticMeshVertexBuffer;
	const FPositionVertexBuffer& Positions = LodData.StaticVertexBuffers.PositionVertexBuffer;

	const FRawStaticIndexBuffer16or32Interface* IndexInterface = LodData.MultiSizeIndexContainer.GetIndexBuffer();

	check(IndexInterface->Num() % 3 == 0);
	const int32 NumTris = IndexInterface->Num() / 3;

	const uint32 NumLodVerts = Weights->GetNumVertices();
	const uint32 MaxInfluences = LodData.GetVertexBufferMaxBoneInfluences();

	if(MaxInfluences == 0)
	{
		return {};
	}

	FBoxSphereBounds3f::Builder BoundsBuilder;
	const int32 NumSections = LodData.RenderSections.Num();

	for(int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		const FSkelMeshRenderSection& Section = LodData.RenderSections[SectionIndex];
		const uint32 SectionMaxInfluences = Section.MaxBoneInfluences;

		for(uint32 TriIndex = 0; TriIndex < Section.NumTriangles; ++TriIndex)
		{
			const uint32 TriBaseIndex = Section.BaseIndex + TriIndex * 3;

			bool bAcceptTriangle = false;
			uint32 ReferenceBoneIndex = 0;
			uint32 ReferenceInfluenceIndex = 0;

			for(int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
			{
				const uint32 VertIndex = IndexInterface->Get(TriBaseIndex + TriVertIndex);

				if(SelectMode == EVertexSelectMode::Any)
				{
					for(uint32 InfluenceIndex = 0; InfluenceIndex < SectionMaxInfluences; InfluenceIndex++)
					{
						const uint32 RenderBoneIndex = Weights->GetBoneIndex(VertIndex, InfluenceIndex);
						const uint32 MeshBoneIndex = Section.BoneMap[RenderBoneIndex];

						uint16 BoneWeight = Weights->GetBoneWeight(VertIndex, InfluenceIndex);

						if(BoneWeight > 0 && RefSkeleton.GetBoneName(MeshBoneIndex) == BoneName)
						{
							bAcceptTriangle = true;
							ReferenceBoneIndex = MeshBoneIndex;
							ReferenceInfluenceIndex = InfluenceIndex;

							// Vertex will only have one influence for a given bone, skip the rest
							break;
						}
					}
				}
				else
				{
					int32 MaxInfluenceIndex = 0;
					uint16 MaxInfluence = Weights->GetBoneWeight(VertIndex, 0);
					for(uint32 InfluenceIndex = 1; InfluenceIndex < SectionMaxInfluences; InfluenceIndex++)
					{
						uint16 BoneWeight = Weights->GetBoneWeight(VertIndex, InfluenceIndex);

						if(BoneWeight > MaxInfluence)
						{
							MaxInfluenceIndex = InfluenceIndex;
							MaxInfluence = BoneWeight;
						}
					}

					const uint32 RenderBoneIndex = Weights->GetBoneIndex(VertIndex, MaxInfluenceIndex);
					const uint32 MeshBoneIndex = Section.BoneMap[RenderBoneIndex];

					if(RefSkeleton.GetBoneName(MeshBoneIndex) == BoneName)
					{
						bAcceptTriangle = true;
						ReferenceBoneIndex = MeshBoneIndex;
						ReferenceInfluenceIndex = MaxInfluenceIndex;
					}
				}

				if(bAcceptTriangle)
				{
					// Once we have one vert we no longer need to check, this triangle is included
					break;
				}
			}

			if(bAcceptTriangle)
			{
				const FMatrix44f& RefMatrix = Mesh->GetRefBasesInvMatrix()[ReferenceBoneIndex];

				const int32 ResultBaseIndex = Result.Positions.Num();
				Result.Triangles.Add({ ResultBaseIndex, ResultBaseIndex + 1, ResultBaseIndex + 2 });

				for(int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
				{
					const uint32 VertIndex = IndexInterface->Get(TriBaseIndex + TriVertIndex);

					Result.Positions.Add(RefMatrix.TransformPosition(Positions.VertexPosition(VertIndex)));
					Result.Normals.Add(RefMatrix.TransformVector(MeshVertexBuffer.VertexTangentZ(VertIndex)));

					// Update bone bounds
					BoundsBuilder += Result.Positions.Last();

					const int32 RefBoneIndex = Section.BoneMap[Weights->GetBoneIndex(VertIndex, ReferenceInfluenceIndex)];
					const FName RefBoneName = RefSkeleton.GetBoneName(RefBoneIndex);

					// Presume zero (if we don't find the bone then it has a zero weight). The checks below are duplicated because
					// this first check should pass for the majority of the verts in the selection - only the boundary verts could
					// potentially require a search through the list of influences, so we quickly check the expected influcence before
					// falling back to a search
					Result.Weights.Add(0);
					if(RefBoneName == BoneName)
					{
						// Influence matches, take weight directly
						Result.Weights.Last() = Weights->GetBoneWeight(VertIndex, ReferenceInfluenceIndex) * WeightsToFloatMult;
					}
					else
					{
						// Need to see if it's on a different influence
						const int32 NumInfluences = Weights->GetMaxBoneInfluences();
						for(int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
						{
							const int32 SectionBoneIndex = Weights->GetBoneIndex(VertIndex, InfluenceIndex);

							if(!Section.BoneMap.IsValidIndex(SectionBoneIndex))
							{
								// Invalid bone index for this influence
								continue;
							}

							const int32 TestBoneIndex = Section.BoneMap[Weights->GetBoneIndex(VertIndex, InfluenceIndex)];
							const FName TestBoneName = RefSkeleton.GetBoneName(TestBoneIndex);

							if(TestBoneName == BoneName)
							{
								Result.Weights.Last() = Weights->GetBoneWeight(VertIndex, InfluenceIndex) * WeightsToFloatMult;
							}
						}
					}
				}
			}
		}
	}

	Result.Bounds = BoundsBuilder;
	return Result;
}

FBoneVertData GetMergedBoneVertexData(const FRigidAssetBoneInfo& Bone, USkeletalMesh* Mesh, int32 Lod, EVertexSelectMode SelectMode)
{
	if(!Mesh)
	{
		return {};
	}

	FBoneVertData RootData = GetBoneVertexData(Bone.Name, Mesh, Lod, SelectMode);

	const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	const TArray<FTransform>& RefTransforms = RefSkeleton.GetRawRefBonePose();

	for(int32 BoneIndex : Bone.MergedChildren)
	{
		FBoneVertData ChildData = GetBoneVertexData(RefSkeleton.GetBoneName(BoneIndex), Mesh, Lod, SelectMode);
		FTransform ToParent = RefTransforms[BoneIndex];

		// Merged bones may be many bones away, need to walk up the chain and combine the transforms
		int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
		while(ParentIndex != Bone.Index && ParentIndex != INDEX_NONE)
		{
			ToParent *= RefTransforms[ParentIndex];
			ParentIndex = RefSkeleton.GetParentIndex(ParentIndex);
		}

		if(ParentIndex == INDEX_NONE)
		{
			// Somehow have a connection to root, with a correctly formed selection this should not be possible
			return {};
		}

		// Track the number of verts before appending so we can rebase the child triangles
		const int32 VertBase = RootData.Positions.Num();
		const int32 TriangleBase = RootData.Triangles.Num();

		RootData.Positions.Reserve(RootData.Positions.Num() + ChildData.Positions.Num());
		RootData.Normals.Reserve(RootData.Normals.Num() + ChildData.Normals.Num());
		for(int32 VertIndex = 0; VertIndex < ChildData.Positions.Num(); ++VertIndex)
		{
			const FVector PositionInParentFrame = ToParent.TransformPosition(FVector(ChildData.Positions[VertIndex]));
			const FVector NormalInParentFrame = ToParent.TransformVector(FVector(ChildData.Normals[VertIndex]));

			RootData.Positions.Add(FVector3f(PositionInParentFrame));
			RootData.Normals.Add(FVector3f(NormalInParentFrame));
		}

		RootData.Triangles.Append(ChildData.Triangles);

		for(int32 TriIndex = TriangleBase; TriIndex < RootData.Triangles.Num(); ++TriIndex)
		{
			RootData.Triangles[TriIndex][0] += VertBase;
			RootData.Triangles[TriIndex][1] += VertBase;
			RootData.Triangles[TriIndex][2] += VertBase;
		}

		RootData.Weights.Append(ChildData.Weights);
		RootData.Bounds = RootData.Bounds + ChildData.Bounds.TransformBy(FTransform3f(ToParent));
	}

	return RootData;
}

FRigidAssetBoneSelection MergeSelection(const FBaseGenerationSettings& Settings, const FRigidAssetBoneSelection& InSelectionSorted)
{
	if(!InSelectionSorted.Mesh)
	{
		return {};
	}

	FRigidAssetBoneSelection Result;

	struct FLocalBoneData
	{
		FRigidAssetBoneInfo Bone;
		FRigidAssetBoneInfo RootBone;

		FBoneVertData VertData;

		bool bRootBone = false;

		TArray<FRigidAssetBoneInfo> ChildBones;
	};

	TObjectPtr<USkeletalMesh> Mesh = InSelectionSorted.Mesh;
	const FReferenceSkeleton& RefSkel = Mesh->GetRefSkeleton();
	const TArray<FTransform>& RefTransforms = RefSkel.GetRawRefBonePose();

	const int32 NumSelectionBones = InSelectionSorted.SelectedBones.Num();
	TArray<FLocalBoneData> BoneData;
	BoneData.AddDefaulted(NumSelectionBones);

	constexpr static int32 SerialThreshold_GetBoneVertexData = 4;
	TArray<UE::Tasks::FTask> WorkerTasks = DispatchTasks<SerialThreshold_GetBoneVertexData>(UE_SOURCE_LOCATION, NumSelectionBones, [Mesh, &BoneData, &InSelectionSorted, &Settings](int32 Index)
		{
			FLocalBoneData& Local = BoneData[Index];
			const FRigidAssetBoneInfo& Bone = InSelectionSorted.SelectedBones[Index];

			Local.Bone = Bone;
			Local.RootBone = Bone;
			Local.VertData = GetBoneVertexData(Bone.Name, Mesh, Settings.SourceLod, Settings.VertexMode);
		});

	UE::Tasks::Wait(WorkerTasks);

	const float MinSizeSq = Settings.Operation == EMergeOperation::MergeSmall ? Settings.MinimumBoneSize * Settings.MinimumBoneSize : std::numeric_limits<float>::max();
	const float MergeThresholdSizeSq = Settings.Operation == EMergeOperation::MergeSmall ? Settings.SmallBoneMergeThresholdOverride * Settings.SmallBoneMergeThresholdOverride : 0;

	TMap<int32, int32> BoneIndexToMergedIndex;

	// Walk backwards, merging child bones into parent
	const int32 NumBones = InSelectionSorted.SelectedBones.Num();
	for(int32 BoneIndex = NumBones - 1; BoneIndex >= 0; --BoneIndex)
	{
		FLocalBoneData& LocalData = BoneData[BoneIndex];
		const FRigidAssetBoneInfo& Bone = LocalData.Bone;

		const float BoneSizeSq = LocalData.VertData.Bounds.BoxExtent.SizeSquared() * 2;

		if(BoneSizeSq < MinSizeSq)
		{
			if(BoneSizeSq < MergeThresholdSizeSq || (Settings.SmallBoneOp == ESmallBoneOperation::Skip && Settings.Operation == EMergeOperation::MergeSmall) || LocalData.VertData.Positions.Num() == 0)
			{
				// Skip this bone
				LocalData.RootBone = {};
				continue;
			}

			const int32 ParentSkelIndex = RefSkel.GetParentIndex(Bone.Index);
			const int32 ParentBoneIndex = InSelectionSorted.SelectedBones.IndexOfByPredicate([ParentSkelIndex](const FRigidAssetBoneInfo& SearchBone)
				{
					return SearchBone.Index == ParentSkelIndex;
				});

			if(ParentBoneIndex != INDEX_NONE)
			{
				FLocalBoneData& ParentLocalData = BoneData[ParentBoneIndex];
				FBoneVertData& ParentVertData = ParentLocalData.VertData;

				// Need to transform all child verts into parent space
				FTransform ToParent = RefTransforms[Bone.Index];

				for(int32 ChildVertIndex = 0; ChildVertIndex < LocalData.VertData.Positions.Num(); ++ChildVertIndex)
				{
					FVector PositionInParentFrame = ToParent.TransformPosition(FVector(LocalData.VertData.Positions[ChildVertIndex]));
					FVector NormalInParentFrame = ToParent.TransformVector(FVector(LocalData.VertData.Normals[ChildVertIndex]));

					ParentVertData.Positions.Add(FVector3f(PositionInParentFrame));
					ParentVertData.Normals.Add(FVector3f(NormalInParentFrame));
				}

				// Weights don't need transforming - just append them
				ParentVertData.Weights.Append(LocalData.VertData.Weights);
				ParentVertData.Bounds = ParentVertData.Bounds + LocalData.VertData.Bounds.TransformBy(FTransform3f(ToParent));

				ParentLocalData.ChildBones.Add(LocalData.RootBone);
				ParentLocalData.ChildBones.Append(LocalData.ChildBones);

				// Clear this bone so we can filter easily after the merge is complete
				LocalData.RootBone = {};
			}
		}
	}

	auto IsMergedBone = [](const FLocalBoneData& Bone)
		{
			return Bone.RootBone.Index == INDEX_NONE;
		};

	BoneData.RemoveAllSwap(IsMergedBone, EAllowShrinking::No);

	// Convert local bone data back to a selection
	Result.Mesh = InSelectionSorted.Mesh;
	Result.Skeleton = InSelectionSorted.Skeleton;

	for(const FLocalBoneData& LocalBone : BoneData)
	{
		Result.SelectedBones.Emplace(LocalBone.Bone);
		FRigidAssetBoneInfo& NewSelectionBone = Result.SelectedBones.Last();

		for(const FRigidAssetBoneInfo& ChildBone : LocalBone.ChildBones)
		{
			NewSelectionBone.MergedChildren.Add(ChildBone.Index);
		}
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TArray<FRigidAssetBoneGeometry> UBoneGeometryGenerator_Box::Build(FRigidAssetBoneSelection Bones)
{
	using namespace UE::Chaos::RigidAsset;

	if(!Bones.Mesh || !Bones.Skeleton)
	{
		return {};
	}

	TArray<FRigidAssetBoneGeometry> Result;

	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		FBoneVertData BoneVerts = GetBoneVertexData(Bone.Name, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

		if(BoneVerts.Positions.Num() == 0)
		{
			// No influences, make a small sphere and warn
			// #TODO pass context through for dataflow warnings
			Result.Emplace(Bone, FSimpleGeometry(MakeShared<FKSphereElem>(5.0f)));

			continue;
		}

		FBox3f VertBox(BoneVerts.Positions);

		TSharedPtr<FKBoxElem> NewBox = MakeShared<FKBoxElem>();
		NewBox->X = VertBox.GetExtent().X * 2.0f;
		NewBox->Y = VertBox.GetExtent().Y * 2.0f;
		NewBox->Z = VertBox.GetExtent().Z * 2.0f;
		NewBox->Center = static_cast<FVector>(VertBox.GetCenter());

		Result.Emplace(Bone, FSimpleGeometry(NewBox));
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TArray<FRigidAssetBoneGeometry> UBoneGeometryGenerator_Sphere::Build(FRigidAssetBoneSelection Bones)
{
	using namespace UE::Chaos::RigidAsset;

	if(!Bones.Mesh || !Bones.Skeleton)
	{
		return {};
	}

	TArray<FRigidAssetBoneGeometry> Result;

	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		FBoneVertData Verts = GetMergedBoneVertexData(Bone, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

		if(Verts.Positions.Num() == 0)
		{
			// No influences, make a small sphere and warn
			// #TODO pass context through for dataflow warnings
			Result.Emplace(Bone, FSimpleGeometry(MakeShared<FKSphereElem>(5.0f)));
			continue;
		}

		FBox3f VertBox(Verts.Positions);

		TSharedPtr<FKSphereElem> NewSphere = MakeShared<FKSphereElem>();
		NewSphere->Radius = VertBox.GetExtent().Length();
		NewSphere->Center = static_cast<FVector>(VertBox.GetCenter());

		Result.Emplace(Bone, FSimpleGeometry(NewSphere));
	}

	return Result;
}

#if WITH_EDITOR
bool UBoneGeometryGenerator_Sphere::CanDebugDraw() const
{
	return true;
}

bool UBoneGeometryGenerator_Sphere::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

void UBoneGeometryGenerator_Sphere::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowNode::FDebugDrawParameters& DebugDrawParameters, FRigidAssetBoneSelection Bones) const
{
	if(!Bones.Mesh || !Bones.Skeleton || !bDrawVerts)
	{
		return;
	}

	TArray<FTransform> BoneTransforms;
	const FReferenceSkeleton& RefSkel = Bones.Mesh->GetRefSkeleton();
	RefSkel.GetBoneAbsoluteTransforms(BoneTransforms);

	int32 BoneColorSeed = 0;

	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		if(DrawVertsForBoneName != NAME_None && Bone.Name != DrawVertsForBoneName)
		{
			continue;
		}

		FBoneVertData BoneVerts = GetMergedBoneVertexData(Bone, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

		DataflowRenderingInterface.SetColor(FLinearColor::MakeRandomSeededColor(BoneColorSeed++));
		for(const FVector3f& Pos : BoneVerts.Positions)
		{
			DataflowRenderingInterface.DrawPoint(BoneTransforms[Bone.Index].TransformPosition(static_cast<FVector>(Pos)));
		}
	}
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace Private
{
	// Reproduced from PhysicsAssetUtils module to enable runtime module compilation
	// TODO: Refector PhyiscsAssetUtils into Runtime and Editor modules to better centralise utils.
	template<typename RealType>
	FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<RealType>, int> PointView)
	{
		using FVectorType = UE::Math::TVector<RealType>;

		if(PointView.Num() == 0)
		{
			return FMatrix::Identity;
		}

		//get average
		const RealType N = PointView.Num();
		FVectorType U = FVectorType::ZeroVector;

		for(const FVectorType& Point : PointView)
		{
			U += Point;
		}

		U = U / N;

		//compute error terms
		TArray<FVectorType> Errors;
		Errors.AddUninitialized(N);

		for(int32 i = 0; i < N; ++i)
		{
			Errors[i] = PointView[i] - U;
		}

		FMatrix Covariance = FMatrix::Identity;
		for(int32 j = 0; j < 3; ++j)
		{
			FVectorType Axis = FVectorType::ZeroVector;
			RealType* Cj = &Axis.X;
			for(int32 k = 0; k < 3; ++k)
			{
				RealType Cjk = 0.f;
				for(int32 i = 0; i < N; ++i)
				{
					const RealType* Error = &Errors[i].X;
					Cj[k] += Error[j] * Error[k];
				}
				Cj[k] /= N;
			}

			Covariance.SetAxis(j, UE::Math::TVector<double>(Axis));
		}

		return Covariance;
	}

	FVector ComputeEigenVector(const FMatrix& A)
	{
		//using the power method: this is ok because we only need the dominate eigenvector and speed is not critical: http://en.wikipedia.org/wiki/Power_iteration
		FVector Bk = FVector(0, 0, 1);
		for(int32 i = 0; i < 32; ++i)
		{
			float Length = Bk.Size();
			if(Length > 0.f)
			{
				Bk = A.TransformVector(Bk) / Length;
			}
		}

		return Bk.GetSafeNormal();
	}
}

TArray<FRigidAssetBoneGeometry> UBoneGeometryGenerator_Capsule::Build(FRigidAssetBoneSelection Bones)
{
	using namespace UE::Chaos::RigidAsset;

	if(!Bones.Mesh || !Bones.Skeleton)
	{
		return {};
	}

	TArray<FRigidAssetBoneGeometry> Result;

	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		FBoneVertData BoneVerts = GetMergedBoneVertexData(Bone, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

		if(BoneVerts.Positions.Num() == 0)
		{
			// No influences, make a small sphere and warn
			// #TODO pass context through for dataflow warnings
			Result.Emplace(Bone, FSimpleGeometry(MakeShared<FKSphereElem>(5.0f)));
			continue;
		}

		// Y and Z are flipped in the default case deliberately to line up a capsule
		// correctly with the bone extent, otherwise it appears on its side
		FVector3f XAxis(1, 0, 0);
		FVector3f YAxis(0, 0, 1);
		FVector3f ZAxis(0, 1, 0);

		if(Alignment == EBodyAlignment::Verts)
		{
			FMatrix CovarianceMat = Private::ComputeViewCovarianceMatrix(MakeConstArrayView(BoneVerts.Positions));
			ZAxis = static_cast<FVector3f>(Private::ComputeEigenVector(CovarianceMat));
			ZAxis.FindBestAxisVectors(XAxis, YAxis);
		}

		auto AveragePosition = [](TArrayView<const FVector3f> Verts)
			{
				FVector3f Avg(0);

				if(Verts.Num() == 0)
				{
					return Avg;
				}

				for(const FVector3f Vert : Verts)
				{
					Avg += Vert;
				}

				return Avg / static_cast<float>(Verts.Num());
			};

		auto AxisInterval = [](TArrayView<const FVector3f> Verts, FVector3f Axis)
			{
				float Min = std::numeric_limits<float>::max();
				float Max = std::numeric_limits<float>::lowest();

				for(const FVector3f Vert : Verts)
				{
					const float Projected = FVector3f::DotProduct(Vert, Axis);
					if(Projected < Min)
					{
						Min = Projected;
					}

					if(Projected > Max)
					{
						Max = Projected;
					}
				}

				return Max - Min;
			};

		TArrayView<const FVector3f> PositionView = MakeConstArrayView(BoneVerts.Positions);
		FVector3f Extent;
		Extent.X = AxisInterval(PositionView, XAxis);
		Extent.Y = AxisInterval(PositionView, YAxis);
		Extent.Z = AxisInterval(PositionView, ZAxis);

		// Work out length and radius, length is the longest extent, radius is half the diagonal of the other two.
		float Length, Radius;
		if(Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			Length = Extent.X;
			Radius = FMath::Sqrt(Extent.Y * Extent.Y + Extent.Z * Extent.Z);
		}
		else if(Extent.Y > Extent.Z)
		{
			Length = Extent.Y;
			Radius = FMath::Sqrt(Extent.X * Extent.X + Extent.Z * Extent.Z);
		}
		else
		{
			Length = Extent.Z;
			Radius = FMath::Sqrt(Extent.X * Extent.X + Extent.Y * Extent.Y);
		}

		FMatrix ElemMatrix(static_cast<FVector>(XAxis),
			static_cast<FVector>(YAxis),
			static_cast<FVector>(ZAxis),
			FVector::ZeroVector);

		TSharedPtr<FKSphylElem> NewCapsule = MakeShared<FKSphylElem>();
		NewCapsule->Center = static_cast<FVector>(AveragePosition(BoneVerts.Positions));
		NewCapsule->Length = Length;
		NewCapsule->Radius = Radius * 0.5f;
		NewCapsule->Rotation = ElemMatrix.Rotator();

		Result.Emplace(Bone, FSimpleGeometry(NewCapsule));
	}

	return Result;
}

#if WITH_EDITOR
bool UBoneGeometryGenerator_Capsule::CanDebugDraw() const
{
	return true;
}

bool UBoneGeometryGenerator_Capsule::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

void UBoneGeometryGenerator_Capsule::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowNode::FDebugDrawParameters& DebugDrawParameters, FRigidAssetBoneSelection Bones) const
{
	if(!Bones.Mesh || !Bones.Skeleton || !bDrawVerts)
	{
		return;
	}

	TArray<FTransform> BoneTransforms;
	const FReferenceSkeleton& RefSkel = Bones.Mesh->GetRefSkeleton();
	RefSkel.GetBoneAbsoluteTransforms(BoneTransforms);

	int32 BoneColorSeed = 0;
	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		if(DrawVertsForBoneName != NAME_None && Bone.Name == DrawVertsForBoneName)
		{
			continue;
		}

		FBoneVertData BoneVerts = GetBoneVertexData(Bone.Name, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

		DataflowRenderingInterface.SetColor(FLinearColor::MakeRandomSeededColor(BoneColorSeed++));
		for(const FVector3f& Pos : BoneVerts.Positions)
		{
			DataflowRenderingInterface.DrawPoint(BoneTransforms[Bone.Index].TransformPosition(static_cast<FVector>(Pos)));
		}
	}
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TArray<FRigidAssetBoneGeometry> UBoneGeometryGenerator_Convex::Build(FRigidAssetBoneSelection Bones)
{
	using namespace UE::Chaos::RigidAsset;

	if(!Bones.Mesh || !Bones.Skeleton)
	{
		return {};
	}

	TArray<FRigidAssetBoneGeometry> Result;

	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		FBoneVertData BoneVerts = GetMergedBoneVertexData(Bone, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

		if(BoneVerts.Positions.Num() == 0)
		{
			// No influences, make a small sphere and warn
			// #TODO pass context through for dataflow warnings
			Result.Emplace(Bone, FSimpleGeometry(MakeShared<FKSphereElem>(5.0f)));
			continue;
		}

		FBox3f VertBox(BoneVerts.Positions);

		TSharedPtr<FKConvexElem> NewConvex = MakeShared<FKConvexElem>();
		TArray<Chaos::FConvexBuilder::FVec3Type> InPositions;
		TArray<Chaos::FConvexBuilder::FPlaneType> Planes;
		TArray<TArray<int32>> Indices;
		TArray<Chaos::FConvexBuilder::FVec3Type> ConvexVerts;
		Chaos::FConvexBuilder::FAABB3Type Aabb;

		InPositions.Reserve(BoneVerts.Positions.Num());
		for(const FVector3f& Vert : BoneVerts.Positions)
		{
			InPositions.Add(Vert);
		}

		Chaos::FConvexBuilder::Build(InPositions, Planes, Indices, ConvexVerts, Aabb);

		NewConvex->VertexData.Reset(ConvexVerts.Num());

		for(const Chaos::FConvexBuilder::FVec3Type& ConvexVert : ConvexVerts)
		{
			NewConvex->VertexData.Add(static_cast<FVector>(ConvexVert));
		}

		NewConvex->UpdateElemBox();

		Result.Emplace(Bone, FSimpleGeometry(NewConvex));
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Wrapper struct to pad out T to cache-line size to avoid false sharing when we set up an output
// array for a multi-threaded algorithm, avoids multiple threads ever reading/writing the same cache line
template<typename T>
struct alignas(PLATFORM_CACHE_LINE_SIZE) TPadded : public T {};

TArray<FRigidAssetBoneGeometry> UBoneGeometryGenerator_ConvexDecomposition::Build(FRigidAssetBoneSelection Bones)
{
	using namespace UE::Chaos::RigidAsset;

	if(!Bones.Mesh || !Bones.Skeleton)
	{
		return {};
	}

	const int32 NumBones = Bones.SelectedBones.Num();

	TArray<TPadded<TArray<FRigidAssetBoneGeometry>>> WorkerResults;
	WorkerResults.AddDefaulted(NumBones);

	constexpr static int32 SerialThreshold_BoneDecompose = 4;
	TArray<UE::Tasks::FTask> DecompTasks = DispatchTasks<SerialThreshold_BoneDecompose>(UE_SOURCE_LOCATION, NumBones, [this, &Bones, &WorkerResults](int32 BoneIndex)
		{
			const FRigidAssetBoneInfo& Bone = Bones.SelectedBones[BoneIndex];

			{
				FBoneVertData BoneVerts = GetMergedBoneVertexData(Bone, Bones.Mesh, BaseSettings.SourceLod, BaseSettings.VertexMode);

				if(BoneVerts.Positions.Num() == 0)
				{
					// No influences, make a small sphere and warn
					// #TODO pass context through for dataflow warnings
					WorkerResults[BoneIndex].Emplace(Bone, FSimpleGeometry(MakeShared<FKSphereElem>(5.0f)));
					return;
				}

				UE::Geometry::FConvexDecomposition3 Decomp;
				UE::Geometry::FConvexDecomposition3::FPreprocessMeshOptions PreprocessOptions;

				// Merge edges by default and thicken up to 1cm if any axis is coplanar (flat mesh)
				PreprocessOptions.bMergeEdges = true;
				PreprocessOptions.ThickenInputAfterHullFailure = 1.0f;

				TArrayView<const FVector3f> PositionView(BoneVerts.Positions);
				TArrayView<const FIntVector3> IndexView(BoneVerts.Triangles);
				Decomp.InitializeFromIndexMesh(PositionView, IndexView, PreprocessOptions);

				Decomp.bTreatAsSolid = Decomp.IsInputSolid();

				if(Method == EDecompositionMethod::Simple)
				{
					Decomp.Compute(FMath::Max(1, NumHulls));
				}
				else
				{
					UE::Geometry::FNegativeSpaceSampleSettings NegativeSpaceSettings;

					NegativeSpaceSettings.ApplyDefaults();
					NegativeSpaceSettings.bOnlyConnectedToHull = bOnlyConnectedToHull;
					NegativeSpaceSettings.MinRadius = NegativeSpaceMinRadius;
					NegativeSpaceSettings.ReduceRadiusMargin = NegativeSpaceTolerance;
					NegativeSpaceSettings.bRequireSearchSampleCoverage = true;
					NegativeSpaceSettings.bAllowSamplesInsideMesh = Decomp.bTreatAsSolid;

					Decomp.MaxConvexEdgePlanes = 4;
					Decomp.bSplitDisconnectedComponents = false;
					Decomp.ConvexEdgeAngleMoreSamplesThreshold = 180;
					Decomp.InitializeNegativeSpace(NegativeSpaceSettings);

					constexpr int32 MaxUnconstrainedSplits = 10000;
					const int32 NumSplits = NegativeSpaceMaxSplits > 0 ? NegativeSpaceMaxSplits : MaxUnconstrainedSplits;

					for(int32 SplitIndex = 0; SplitIndex < NumSplits; ++SplitIndex)
					{
						int32 NumPerformedSplits = Decomp.SplitWorst(false, 0, true, NegativeSpaceTolerance * 0.5);

						if(NumPerformedSplits == 0)
						{
							// No more splits possible - decomp is complete
							break;
						}

						if(SplitIndex == MaxUnconstrainedSplits - 1)
						{
							UE_LOGF(LogBoneGeometryGenerators, Warning, "Decomposition ran maximum number of splits (%d), potentially stuck in a splitting loop", MaxUnconstrainedSplits);
						}
					}

					Decomp.FixHullOverlapsInNegativeSpace();

					UE::Geometry::FConvexDecomposition3::FMergeSettings MergeSettings;
					MergeSettings.TargetNumParts = NegativeSpaceMaxSplits > 0 ? NegativeSpaceMaxSplits : -1;
					MergeSettings.MinThicknessTolerance = MinThicknessTolerance;
					MergeSettings.bAllowCompact = true;

					Decomp.MergeBest(MergeSettings);
				}

				const int32 NumComputedHulls = Decomp.NumHulls();
				WorkerResults[BoneIndex].Reserve(NumComputedHulls);

				for(int32 HullIndex = 0; HullIndex < NumComputedHulls; ++HullIndex)
				{
					TSharedPtr<FKConvexElem> NewConvex = MakeShared<FKConvexElem>();

					if(bSimplifyHulls)
					{
						UE::Geometry::FDynamicMesh3 HullMesh = Decomp.GetHullMesh(HullIndex);

						UE::Geometry::FMeshConvexHull::FSimplifyOptions Options{
							SimplifyTargetMaxFaces,
							SimplifyGeometricToleranceAfterTarget
						};

						if(!ensure(UE::Geometry::FMeshConvexHull::SimplifyHull(HullMesh, Options)))
						{
							UE_LOGF(LogBoneGeometryGenerators, Error, "Simplification failed for convex hull %d of %d, skipping", HullIndex, NumComputedHulls);
							continue;
						}

						for(int32 MeshVertIndex : HullMesh.VertexIndicesItr())
						{
							NewConvex->VertexData.Add(static_cast<FVector>(HullMesh.GetVertex(MeshVertIndex)));
						}
					}
					else
					{
						TArray<FVector3f> DecompVerts = Decomp.GetVertices<float>(HullIndex);
						for(const FVector3f& Vert : DecompVerts)
						{
							NewConvex->VertexData.Add(static_cast<FVector>(Vert));
						}
					}

					NewConvex->UpdateElemBox();

					WorkerResults[BoneIndex].Emplace(Bone, FSimpleGeometry(NewConvex));
				}
			}
		});
	UE::Tasks::Wait(DecompTasks);

	TArray<FRigidAssetBoneGeometry> Result;

	const int32 TotalNumGeometry = Algo::TransformAccumulate(WorkerResults, [](const TArray<FRigidAssetBoneGeometry>& Item) { return Item.Num(); }, 0);
	Result.Reserve(TotalNumGeometry);

	for(const TArray<FRigidAssetBoneGeometry>& BoneGeometry : WorkerResults)
	{
		Result.Append(BoneGeometry);
	}

	return Result;
}
