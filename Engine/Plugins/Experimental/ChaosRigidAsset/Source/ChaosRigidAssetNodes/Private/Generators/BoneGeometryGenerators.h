// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneSelection.h"
#include "RigidDataflowNode.h"
#include "ShapeElemNodes.h"

#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"

#include "BoneGeometryGenerators.generated.h"

#pragma once

DECLARE_LOG_CATEGORY_EXTERN(LogBoneGeometryGenerators, Log, All);

namespace UE::Dataflow
{
	void RegisterBoneGeometryGeneratorNodes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** 
 * Merge operation to perform
 */
UENUM()
enum class EMergeOperation
{
	MergeAll,
	MergeSmall
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Operation to use for small bones
 * Skip will ignore a bone, and all of its geometry when attempting to generate collision shapes
 * Merge will use the corresponding merge thresholds in the generation settings to decide whether to roll bones up
 * as a single geometry and use all of their render data as a source for the generation process
 */
UENUM()
enum class ESmallBoneOperation
{
	Skip,
	Merge
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Operation for vertex extraction
 * Any will extract all verts that have an influence for the selected bone
 * DominantOnly will only extract verts that have the selcted bone as the dominant weight
 */
UENUM()
enum class EVertexSelectMode
{
	Any,
	DominantOnly
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Whether to include merged bones in returned vertext data
 */
 UENUM()
enum class EVertexMergeMode
{
	RootOnly,
	Merged
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * How to align a body during generation
 * Bone will align along the bone direction and wrap the shape around the verts
 * Verts will align along the major axis of the selected verts
 */
UENUM()
enum class EBodyAlignment
{
	Bone,
	Verts
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Generic configuration for all generator types
 */
USTRUCT()
struct FBaseGenerationSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Generation")
	EMergeOperation Operation = EMergeOperation::MergeSmall;

	// Bones under this limit will be classified as a "Small" bone and apply the requested operation
	UPROPERTY(EditAnywhere, Category = "Generation", meta=(EditCondition="Operation==EMergeOperation::MergeSmall", EditConditionHides, Units=cm))
	float MinimumBoneSize = 20.0f;

	// Which operation to apply to small bones
	UPROPERTY(EditAnywhere, Category = "Generation", meta = (EditCondition = "Operation==EMergeOperation::MergeSmall", EditConditionHides))
	ESmallBoneOperation SmallBoneOp = ESmallBoneOperation::Skip;

	// For the "Merge" small-bone operation - only merge over this threshold and skip otherwise
	UPROPERTY(EditAnywhere, Category = "Generation", meta = (EditCondition = "Operation==EMergeOperation::MergeSmall", EditConditionHides, Units=cm))
	float SmallBoneMergeThresholdOverride = 0.0f;

	// Vertex extraction operation, see EVertexSelectMode
	UPROPERTY(EditAnywhere, Category = "Generation")
	EVertexSelectMode VertexMode = EVertexSelectMode::DominantOnly;

	// The render LOD to use for geometry generation, render data will be extracted from this LOD
	UPROPERTY(EditAnywhere, Category = "Generation")
	int32 SourceLod = 0;

	// Extra thickness added to generated geometry after the genration process produces a geometry
	UPROPERTY(EditAnywhere, Category = "Generation", meta=(Units=cm))
	float Thickness = 0.0f;
};

/**
 * Data extracted from a bone for body generation
 */
struct FBoneVertData
{
	TArray<FVector3f> Positions;
	TArray<FVector3f> Normals;
	TArray<float> Weights;
	TArray<FIntVector3> Triangles;
	FBoxSphereBounds3f Bounds;
};

struct FRigidAssetBoneGeometry
{
	FRigidAssetBoneInfo Bone;
	UE::Chaos::RigidAsset::FSimpleGeometry Geometry;
};

/**
 * Given a bone selection, produce a set of merged bones according the the specified generation settings
 * @param Settings Generation setting to apply for the merge algorithm
 * @param InSelectionSorted A pre-sorted selection of bones
 */
FRigidAssetBoneSelection MergeSelection(const FBaseGenerationSettings& Settings, const FRigidAssetBoneSelection& InSelectionSorted);

/**
 * Base geometry generator for each render->physics generation type.
 */
UCLASS(EditInlineNew, Abstract)
class UBoneGeometryGenerator : public UObject
{
	GENERATED_BODY()

public:

	virtual TArray<FRigidAssetBoneGeometry> Build(FRigidAssetBoneSelection Bones)
	{
		return {};
	}

#if WITH_EDITOR
	virtual bool CanDebugDraw() const;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowNode::FDebugDrawParameters& DebugDrawParameters, FRigidAssetBoneSelection Bones) const;
#endif

protected:

	// Base settings for all generation types
	UPROPERTY(EditAnywhere, Category = "Generation", meta = (EditInline))
	FBaseGenerationSettings BaseSettings;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Basic box generator
 * Takes the extracted verts and wraps a tight box around them
 */
UCLASS()
class UBoneGeometryGenerator_Box : public UBoneGeometryGenerator
{
	GENERATED_BODY()

public:

	TArray<FRigidAssetBoneGeometry> Build(FRigidAssetBoneSelection Bones) final override;

private:
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Basic sphere generator
 * Takes the extracted verts and wraps a sphere around them
 */
UCLASS()
class UBoneGeometryGenerator_Sphere : public UBoneGeometryGenerator
{
	GENERATED_BODY()

public:

	TArray<FRigidAssetBoneGeometry> Build(FRigidAssetBoneSelection Bones) final override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowNode::FDebugDrawParameters& DebugDrawParameters, FRigidAssetBoneSelection Bones) const override;
#endif

private:

	UPROPERTY(EditAnywhere, Category = Settings)
	bool bUseGeometry = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDrawVerts = true;

	UPROPERTY(EditAnywhere, Category=Debug)
	FName DrawVertsForBoneName = NAME_None;
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Capsule generator
 * Takes the extracted verts and wraps capsule around them
 */
UCLASS()
class UBoneGeometryGenerator_Capsule : public UBoneGeometryGenerator
{
	GENERATED_BODY()

public:

	TArray<FRigidAssetBoneGeometry> Build(FRigidAssetBoneSelection Bones) final override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowNode::FDebugDrawParameters& DebugDrawParameters, FRigidAssetBoneSelection Bones) const override;
#endif

private:

	UPROPERTY(EditAnywhere, Category=Geometry)
	EBodyAlignment Alignment = EBodyAlignment::Bone;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDrawVerts = true;

	UPROPERTY(EditAnywhere, Category=Debug)
	FName DrawVertsForBoneName = NAME_None;
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Convex generator
 * Takes the extracted verts then uses them to generate a single convex hull
 */
UCLASS()
class UBoneGeometryGenerator_Convex : public UBoneGeometryGenerator
{
	GENERATED_BODY()

public:

	TArray<FRigidAssetBoneGeometry> Build(FRigidAssetBoneSelection Bones) final override;

private:
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UENUM()
enum class EDecompositionMethod
{
	Simple,
	NegativeSpace
};

/**
 * Convex decomposition generator
 * Takes the extracted verts then uses them to generate a convex hull decomposition of multiple hulls
 */
UCLASS()
class UBoneGeometryGenerator_ConvexDecomposition : public UBoneGeometryGenerator
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Geometry)
	EDecompositionMethod Method = EDecompositionMethod::Simple;

	/** Number of hulls to create in the Simple decomposition mode */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="Method == EDecompositionMethod::Simple", EditConditionHides))
	int32 NumHulls = 4;

	/** When protecting negative space, only look for space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="Method == EDecompositionMethod::NegativeSpace", EditConditionHides))
	bool bOnlyConnectedToHull = true;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="Method == EDecompositionMethod::NegativeSpace", EditConditionHides))
	float NegativeSpaceTolerance = 2.0f;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="Method == EDecompositionMethod::NegativeSpace", EditConditionHides))
	float NegativeSpaceMinRadius = 10.0f;

	/** 
	 * Number of splits to target in the final decomposition when using the negative space method.
	 * If 0 the number of splits will be driven by the decomposition and may produce a more accurate result but take longer.
	 * If >0 the number of splits will be truncated at the specified value
	 */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="Method == EDecompositionMethod::NegativeSpace", EditConditionHides))
	int32 NegativeSpaceMaxSplits = 0;

	/** Optional tolerance (in cm) for merging after decomposition. Hulls under this size will always be merged */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="Method == EDecompositionMethod::NegativeSpace", EditConditionHides))
	double MinThicknessTolerance = 0;

	/** Whether to simplify the resulting hull meshes */
	UPROPERTY(EditAnywhere, Category = Geometry)
	bool bSimplifyHulls = false;

	/** The number of faces to target during simplification */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="bSimplifyHulls", EditConditionHides))
	int32 SimplifyTargetMaxFaces = 128;

	/** If >0 the mesh will be further simplified beloc SimplifyTargetMaxFaces if the mesh error can be kept below this limit */
	UPROPERTY(EditAnywhere, Category = Geometry, meta = (EditCondition="bSimplifyHulls", EditConditionHides))
	double SimplifyGeometricToleranceAfterTarget = 0;

	TArray<FRigidAssetBoneGeometry> Build(FRigidAssetBoneSelection Bones) final override;

private:
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Make nodes for each generator, each creates a configured generator that can be wired into the generation node
 */
USTRUCT()
struct FMakeBoxBoneGeometryGenerator : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeBoxBoneGeometryGenerator, "Make Box Bone Geometry Builder", "PhysicsAsset", "")

public:

	FMakeBoxBoneGeometryGenerator(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Generator", meta = (DataflowOutput))
	TObjectPtr<UBoneGeometryGenerator> Generator;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMakeSphereBoneGeometryGenerator : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSphereBoneGeometryGenerator, "Make Sphere Bone Geometry Builder", "PhysicsAsset", "")

public:

	FMakeSphereBoneGeometryGenerator(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Generator", meta = (DataflowOutput))
	TObjectPtr<UBoneGeometryGenerator> Generator;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMakeCapsuleBoneGeometryGenerator : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCapsuleBoneGeometryGenerator, "Make Capsule Bone Geometry Builder", "PhysicsAsset", "")

public:

	FMakeCapsuleBoneGeometryGenerator(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Generator", meta = (DataflowOutput))
	TObjectPtr<UBoneGeometryGenerator> Generator;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMakeConvexBoneGeometryGenerator : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeConvexBoneGeometryGenerator, "Make Convex Bone Geometry Builder", "PhysicsAsset", "")

public:

	FMakeConvexBoneGeometryGenerator(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Generator", meta = (DataflowOutput))
	TObjectPtr<UBoneGeometryGenerator> Generator;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMakeConvexDecompBoneGeometryGenerator : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeConvexDecompBoneGeometryGenerator, "Make Convex Decomposition Bone Geometry Builder", "PhysicsAsset", "")

public:

	FMakeConvexDecompBoneGeometryGenerator(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Generator", meta = (DataflowOutput))
	TObjectPtr<UBoneGeometryGenerator> Generator;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////
