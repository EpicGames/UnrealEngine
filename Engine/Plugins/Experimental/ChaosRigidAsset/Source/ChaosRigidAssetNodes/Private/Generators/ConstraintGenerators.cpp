// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/ConstraintGenerators.h"

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Engine/SkeletalMesh.h"
#include "ReferenceSkeleton.h"

namespace UE::Dataflow
{
	void RegisterConstraintGeneratorNodes()
	{
		// Constraint Generators
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSwingTwistConstraintGenerator);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

EAngularConstraintMotion GetMotionType(float InLimit)
{
	if(InLimit == 0)
	{
		return ACM_Locked;
	}

	if(InLimit >= 180.0f)
	{
		return ACM_Free;
	}

	return ACM_Limited;
}

TArray<TObjectPtr<UPhysicsConstraintTemplate>> UConstraintGenerator_SwingTwist::Build(TObjectPtr<UPhysicsConstraintTemplate> ConstraintTemplate, FRigidAssetBoneSelection Bones) const
{
	TArray<TObjectPtr<UPhysicsConstraintTemplate>> Result;

	// Look at the requested bones and create a constraint to the first parent we find for each
	const FReferenceSkeleton& RefSkel = Bones.Mesh->GetRefSkeleton();
	for(const FRigidAssetBoneInfo& Bone : Bones.SelectedBones)
	{
		int32 CurrParent = RefSkel.GetParentIndex(Bone.Index);

		while(CurrParent != INDEX_NONE)
		{
			const FRigidAssetBoneInfo* ParentBone = Bones.SelectedBones.FindByPredicate([&CurrParent](const FRigidAssetBoneInfo& Item)
				{
					return Item.Index == CurrParent;
				});

			if(!ParentBone)
			{
				CurrParent = RefSkel.GetParentIndex(CurrParent);
				continue;
			}

			// Now we have a parent, and it's in the selection - add a constraint
			TObjectPtr<UPhysicsConstraintTemplate> NewConstraint = NewObject<UPhysicsConstraintTemplate>();
			FConstraintInstance& Defaults = NewConstraint->DefaultInstance;

			if(ConstraintTemplate)
			{
				Defaults.CopyConstraintParamsFrom(&ConstraintTemplate->DefaultInstance);
			}

			Defaults.SetAngularSwing1Limit(GetMotionType(Swing1Limit), Swing1Limit);
			Defaults.SetAngularSwing2Limit(GetMotionType(Swing2Limit), Swing2Limit);
			Defaults.SetAngularTwistLimit(GetMotionType(TwistLimit), TwistLimit);

			Defaults.JointName = Bone.Name;
			Defaults.ConstraintBone1 = Bone.Name;
			Defaults.ConstraintBone2 = ParentBone->Name;

			FTransform ParentTm = CalculateRelativeBoneTransform(Bone.Name, ParentBone->Name, RefSkel);
			FTransform ChildTm = FTransform::Identity;

			Defaults.SetRefPosition(EConstraintFrame::Frame1, ChildTm.GetTranslation());
			Defaults.SetRefOrientation(EConstraintFrame::Frame1, ChildTm.GetUnitAxis(EAxis::X), ChildTm.GetUnitAxis(EAxis::Y));
			Defaults.SetRefPosition(EConstraintFrame::Frame2, ParentTm.GetTranslation());
			Defaults.SetRefOrientation(EConstraintFrame::Frame2, ParentTm.GetUnitAxis(EAxis::X), ParentTm.GetUnitAxis(EAxis::Y));

#if WITH_EDITOR
			// In editor context, set the default instance for the constraint. This is not available outside of editor
			NewConstraint->SetDefaultProfile(Defaults);
#endif

			Result.Add(NewConstraint);

			break;
		}
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMakeSwingTwistConstraintGenerator::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Generator))
	{
		TObjectPtr<UConstraintGenerator> OutGenerator = CastChecked<UConstraintGenerator>(StaticDuplicateObject(Generator, GetTransientPackage()));
		SetValue(Context, OutGenerator, &Generator);
	}
}

void FMakeSwingTwistConstraintGenerator::Register()
{
	AddOutputs(Generator);

	Generator = NewObject<UConstraintGenerator_SwingTwist>(GetOwner());
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////
