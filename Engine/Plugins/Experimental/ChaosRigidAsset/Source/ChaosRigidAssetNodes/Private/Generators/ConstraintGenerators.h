// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "BoneSelection.h"
#include "RigidDataflowNode.h"

#include "ConstraintGenerators.generated.h"

namespace UE::Dataflow
{
	void RegisterConstraintGeneratorNodes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Shared settings for all generator types
 */
USTRUCT()
struct FBaseConstraintGenerationSettings
{
	GENERATED_BODY();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Base class for constraint generators
 */
UCLASS(EditInlineNew, Abstract)
class UConstraintGenerator : public UObject
{
	GENERATED_BODY()

public:

	virtual TArray<TObjectPtr<UPhysicsConstraintTemplate>> Build(TObjectPtr<UPhysicsConstraintTemplate> ConstraintTemplate, FRigidAssetBoneSelection Bones) const
	{ 
		return {};
	}

private:

	UPROPERTY(EditAnywhere, Category = "Generation", meta=(EditInline))
	FBaseConstraintGenerationSettings BaseSettings;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Swing/Twist constraint generator
 * Given limits for each of the angular axes, this generator will link all the bones in the input set together with the specified limits
 */
UCLASS()
class UConstraintGenerator_SwingTwist : public UConstraintGenerator
{
	GENERATED_BODY()

public:

	TArray<TObjectPtr<UPhysicsConstraintTemplate>> Build(TObjectPtr<UPhysicsConstraintTemplate> ConstraintTemplate, FRigidAssetBoneSelection Bones) const override;

private:

	UPROPERTY(EditAnywhere, Category=Constraint)
	float Swing1Limit;

	UPROPERTY(EditAnywhere, Category=Constraint)
	float Swing2Limit;

	UPROPERTY(EditAnywhere, Category=Constraint)
	float TwistLimit;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Generator make nodes
 */
USTRUCT()
struct FMakeSwingTwistConstraintGenerator : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSwingTwistConstraintGenerator, "Make Swing/Twist Constraint Generator", "PhysicsAsset", "")

public:

	FMakeSwingTwistConstraintGenerator(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Generator", meta = (DataflowOutput))
	TObjectPtr<UConstraintGenerator> Generator;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////
