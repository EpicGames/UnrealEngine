// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/SimulationAffector.h"

#include "Chaos/ParticleHandle.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Math/UnitConversion.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsActorHandle_Chaos.h"


namespace UE::Dataflow
{
	void RegisterSimulationAffectorNodes()
	{
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowSimulationAffectorTypes);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSimulationAccelerationAffectorNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSimulationWindAffectorNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowSimulationAccelerationAffector::Process(const TArray<ImmediatePhysics::FActorHandle*>& Actors) const
{
	for (ImmediatePhysics::FActorHandle* Actor : Actors)
	{
		if (Actor && Actor->IsSimulated())
		{
			Actor->AddForce(Acceleration, ImmediatePhysics::EForceType::AddAcceleration);
		}
	}
}

FDataflowSimulationAccelerationAffectorNode::FDataflowSimulationAccelerationAffectorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Affector);
}

void FDataflowSimulationAccelerationAffectorNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	check(Out->IsA(&Affector));
	SetValue(Context, Affector, &Affector);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowSimulationWindAffector::Process(const TArray<ImmediatePhysics::FActorHandle*>& Actors) const
{
	const FVector NormalizedWindDirection = WindDirection.GetSafeNormal(0, FVector::XAxisVector);
	const float WindSpeedInCmPerSecond = FUnitConversion::Convert(WindSpeed, EUnit::KilometersPerHour, EUnit::CentimetersPerSecond);
	const float AirDensityInKgPerCm3 = FUnitConversion::Convert(AirDensity, EUnit::KilogramsPerCubicMeter, EUnit::KilogramsPerCubicCentimeter);
	const FVector WindVelocitySquared = NormalizedWindDirection * ( WindSpeedInCmPerSecond * WindSpeedInCmPerSecond);
	const FVector WindPressure = (WindVelocitySquared * AirDensityInKgPerCm3 * 0.5f);

	for (ImmediatePhysics::FActorHandle* Actor : Actors)
	{
		if (Actor && Actor->IsSimulated())
		{
			if (const Chaos::FGeometryParticleHandle* Particle = Actor->GetParticle())
			{
				const FTransform ActorTransform = Actor->GetWorldTransform();
				// gross estimation the wind facing area 
				const float TotalVolumeInCm3 = Particle->LocalBounds().GetVolume();
				const float CubeSizeInCm = FMath::Pow(TotalVolumeInCm3, 1.0 / 3.0);
				const float EstimatedAreaInCm2 = CubeSizeInCm * CubeSizeInCm;

				// then estimate how much we are facing the wind
				const float DotProductWithZAxis = ActorTransform.GetUnitAxis(EAxis::Z).Dot(NormalizedWindDirection);
				const float FacingFactor = FMath::Clamp(1.0 - FMath::Abs(DotProductWithZAxis), 0.0, 1.0);

				const FVector WindForce = (EstimatedAreaInCm2 * FacingFactor) * WindPressure;
				Actor->AddForce(WindForce);
			}
		}
	}
}

FDataflowSimulationWindAffectorNode::FDataflowSimulationWindAffectorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Affector);
}

void FDataflowSimulationWindAffectorNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	check(Out->IsA(&Affector));
	SetValue(Context, Affector, &Affector);
}