// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"

#include "SimulationAffector.generated.h"

USTRUCT()
struct FDataflowSimulationAffector
{
	GENERATED_BODY();
public:
	virtual ~FDataflowSimulationAffector() = default;
	virtual void Process(const TArray<ImmediatePhysics::FActorHandle*>& Actors) const {};
};

/** UStruct based types */
USTRUCT()
struct FDataflowSimulationAffectorTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowUDerivedStructTypePolicy<FDataflowSimulationAffector>;
	using FStorageType = void;
	GENERATED_BODY()
};

namespace UE::Dataflow
{
	void RegisterSimulationAffectorNodes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FDataflowSimulationAccelerationAffector: public FDataflowSimulationAffector
{
	GENERATED_BODY();

	virtual ~FDataflowSimulationAccelerationAffector() override = default;
	virtual void Process(const TArray<ImmediatePhysics::FActorHandle*>& Actors) const override;

private:
	UPROPERTY(EditAnywhere, Category = Physics, meta = (ForceUnits = "cm/s2"))
	FVector Acceleration = FVector(0, 0, -980); // default to earth gravity
};

/**
 * Output a affector that apply a constant acceleration to all dynamic actors
 */
USTRUCT()
struct FDataflowSimulationAccelerationAffectorNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSimulationAccelerationAffectorNode, "AccelerationAffector", "PhysicsAsset", "")

public:
	FDataflowSimulationAccelerationAffectorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(EditAnywhere, Category = Affector, meta = (DataflowOutput, ShowOnlyInnerProperties))
	FDataflowSimulationAccelerationAffector Affector;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FDataflowSimulationWindAffector : public FDataflowSimulationAffector
{
	GENERATED_BODY();

	virtual ~FDataflowSimulationWindAffector() override = default;
	virtual void Process(const TArray<ImmediatePhysics::FActorHandle*>& Actors) const override;

private:
	UPROPERTY(EditAnywhere, Category = Physics, meta = (ForceUnits = "kg/m3"))
	float AirDensity = 1.2f; // standard air density at about 20C

	UPROPERTY(EditAnywhere, Category = Physics, meta = (ForceUnits = "km/h"))
	float WindSpeed = 15.f;

	UPROPERTY(EditAnywhere, Category = Physics)
	FVector WindDirection = FVector::XAxisVector;
};

/**
 * Output a affector that apply a Wind force onto the rigid bodies
 */
USTRUCT()
struct FDataflowSimulationWindAffectorNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSimulationWindAffectorNode, "WindAffector", "PhysicsAsset", "")

public:
	FDataflowSimulationWindAffectorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(EditAnywhere, Category = Affector, meta = (DataflowOutput, ShowOnlyInnerProperties))
	FDataflowSimulationWindAffector Affector;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};