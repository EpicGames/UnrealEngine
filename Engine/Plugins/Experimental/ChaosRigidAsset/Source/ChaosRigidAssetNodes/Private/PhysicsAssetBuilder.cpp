// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetBuilder.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/UObjectIterator.h"
#include "Components/SkeletalMeshComponent.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#endif

namespace UE::Chaos::RigidAsset
{
	FPhysicsAssetBuilder FPhysicsAssetBuilder::Make(TObjectPtr<USkeleton> InTargetSkeleton)
	{
		FPhysicsAssetBuilder Builder;
		Builder.TargetSkeleton = InTargetSkeleton;

		return Builder;
	}

	FPhysicsAssetBuilder FPhysicsAssetBuilder::Make(TObjectPtr<USkeleton> InTargetSkeleton, const TArray<TObjectPtr<USkeletalBodySetup>>& InBodies, const TArray<TObjectPtr<UPhysicsConstraintTemplate>>& InConstraints)
	{
		FPhysicsAssetBuilder Builder;

		Builder.TargetSkeleton = InTargetSkeleton;
		Builder.Bodies = InBodies;
		Builder.Constraints = InConstraints;

		return Builder;
	}

	FPhysicsAssetBuilder& FPhysicsAssetBuilder::Body(TObjectPtr<USkeletalBodySetup> NewBody)
	{
		Bodies.Add(NewBody);

		return *this;
	}

	FPhysicsAssetBuilder& FPhysicsAssetBuilder::Joint(TObjectPtr<UPhysicsConstraintTemplate> NewJoint)
	{
		Constraints.Add(NewJoint);

		return *this;
	}

	FPhysicsAssetBuilder& FPhysicsAssetBuilder::JoinLast(TObjectPtr<UPhysicsConstraintTemplate> NewJoint)
	{
		// We need two bodies to join
		const int32 NumBodies = Bodies.Num();
		if(NumBodies > 1)
		{
			TObjectPtr<USkeletalBodySetup> BodiesToJoin[] = { Bodies[NumBodies - 2], Bodies[NumBodies - 1] };

			NewJoint->DefaultInstance.ConstraintBone1 = BodiesToJoin[0]->BoneName;
			NewJoint->DefaultInstance.ConstraintBone2 = BodiesToJoin[1]->BoneName;

			Constraints.Add(NewJoint);
		}
		else
		{
			UE_LOGF(LogTemp, Error, "JoinLast called with less than two available bodies in the builder - cannot join these bodies");
		}

		return *this;
	}

	FPhysicsAssetBuilder& FPhysicsAssetBuilder::Path(FString InDesiredPath)
	{
		DesiredPath = InDesiredPath;

		return *this;
	}

	FPhysicsAssetBuilder& FPhysicsAssetBuilder::Path(TObjectPtr<UObject> InObjectReferenceForPath)
	{
		DesiredPath = InObjectReferenceForPath->GetPathName();

		return *this;
	}

	FPhysicsAssetBuilder& FPhysicsAssetBuilder::SetTargetAsset(TObjectPtr<UPhysicsAsset> InAsset)
	{
		TargetAsset = InAsset;

		return *this;
	}

	TObjectPtr<UPhysicsAsset> FPhysicsAssetBuilder::Build()
	{
		if(!TargetAsset)
		{
			CreateNewAsset();

			// If we fail to make an asset, abort
			if(!TargetAsset)
			{
				return nullptr;
			}
		}

		// Need to replace the objects already in the asset, first move them to the transient package to get them out of the physics asset package
		for(TObjectPtr<USkeletalBodySetup> BodyToRename : TargetAsset->SkeletalBodySetups)
		{
			verify(BodyToRename->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors));
		}

		for(TObjectPtr<UPhysicsConstraintTemplate> ConstraintToRename : TargetAsset->ConstraintSetup)
		{
			verify(ConstraintToRename->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors));
		}

		// Swap to new setups and rename them into the asset package
		TargetAsset->SkeletalBodySetups = Bodies;
		TargetAsset->ConstraintSetup = Constraints;

		for(TObjectPtr<USkeletalBodySetup> BodyToRename : TargetAsset->SkeletalBodySetups)
		{
			verify(BodyToRename->Rename(nullptr, TargetAsset, REN_DontCreateRedirectors));
		}

		for(TObjectPtr<UPhysicsConstraintTemplate> ConstraintToRename : TargetAsset->ConstraintSetup)
		{
			verify(ConstraintToRename->Rename(nullptr, TargetAsset, REN_DontCreateRedirectors));
		}

		TargetAsset->UpdateBodySetupIndexMap();
		TargetAsset->UpdateBoundsBodiesArray();

		// Disable collisions between directly constrained joints
		for(TObjectPtr<UPhysicsConstraintTemplate> Constraint : TargetAsset->ConstraintSetup)
		{
			const int32 Indices[] = {
				TargetAsset->FindBodyIndex(Constraint->DefaultInstance.ConstraintBone1),
				TargetAsset->FindBodyIndex(Constraint->DefaultInstance.ConstraintBone2)
			};

			if(Indices[0] == INDEX_NONE || Indices[1] == INDEX_NONE)
			{
				continue;
			}

			TargetAsset->DisableCollision(Indices[0], Indices[1]);
		}

		// Refresh components using this physics asset
		for(FThreadSafeObjectIterator Iter(USkeletalMeshComponent::StaticClass()); Iter; ++Iter)
		{
			USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(*Iter);

			if(!SkeletalMeshComponent || !SkeletalMeshComponent->IsRegistered())
			{
				// If the component is invalid, or it's not registered then it can't have physics state.
				continue;
			}

			if(SkeletalMeshComponent->GetPhysicsAsset() == TargetAsset)
			{
				if(SkeletalMeshComponent->IsPhysicsStateCreated())
				{
					SkeletalMeshComponent->RecreatePhysicsState();
					SkeletalMeshComponent->UpdateHasValidBodies();
				}
			}
		}

		return TargetAsset;
	}

	FPhysicsAssetBuilder::FPhysicsAssetBuilder()
	{
	}

	void FPhysicsAssetBuilder::CreateNewAsset()
	{
#if WITH_EDITOR
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		FString UniquePackageName;
		FString UniqueAssetName;

		if(DesiredPath.IsEmpty())
		{
			DesiredPath = "/";
		}

		AssetTools.CreateUniqueAssetName(DesiredPath, TEXT(""), UniquePackageName, UniqueAssetName);

		TargetAsset = CastChecked<UPhysicsAsset>(AssetTools.CreateAsset(*UniqueAssetName, *UniquePackageName, UPhysicsAsset::StaticClass(), nullptr));
#else
		ensureMsgf(false, TEXT("Attempted to create a new asset package outside of the editor. At runtime use SetTargetAsset in order to use FPhysicsAssetBuilder"));
#endif
	}
}
