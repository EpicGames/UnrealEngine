// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Containers/Array.h"

class UObject;
class USkeleton;
class USkeletalBodySetup;
class UPhysicsConstraintTemplate;
class UPhysicsAsset;

namespace UE::Chaos::RigidAsset
{
	/**
	 * Builder for physics assets
	 * Takes source data for physics assets (bodies, constraints, mesh, skeleton) and creates a UPhysicsAsset (or writes to a target asset)
	 */
	class FPhysicsAssetBuilder
	{
	public:

		// Overloads for creating a builder from required data.
		static FPhysicsAssetBuilder Make(TObjectPtr<USkeleton> InTargetSkeleton);
		static FPhysicsAssetBuilder Make(TObjectPtr<USkeleton> InTargetSkeleton, const TArray<TObjectPtr<USkeletalBodySetup>>& InBodies, const TArray<TObjectPtr<UPhysicsConstraintTemplate>>& InConstraints);

		// Append a new body to the asset
		FPhysicsAssetBuilder& Body(TObjectPtr<USkeletalBodySetup> NewBody);

		// Append a new constraint to the asset
		FPhysicsAssetBuilder& Joint(TObjectPtr<UPhysicsConstraintTemplate> NewJoint);

		// Apply the provided constraint template to the last pair of bodies in the builder
		FPhysicsAssetBuilder& JoinLast(TObjectPtr<UPhysicsConstraintTemplate> NewJoint);

		// If not targetting a specific asset, set the path to where the resulting asset will be stored
		FPhysicsAssetBuilder& Path(FString InDesiredPath);
		FPhysicsAssetBuilder& Path(TObjectPtr<UObject> InObjectReferenceForPath);

		// Set a target asset instead of a path - this asset will be overwritten with new state when Build() is called
		FPhysicsAssetBuilder& SetTargetAsset(TObjectPtr<UPhysicsAsset> InAsset);

		// Finalize the physics asset and apply the builder to it
		TObjectPtr<UPhysicsAsset> Build();

	private:

		FPhysicsAssetBuilder();

		void CreateNewAsset();

		FString DesiredPath;

		TObjectPtr<USkeleton> TargetSkeleton;
		TArray<TObjectPtr<USkeletalBodySetup>> Bodies;
		TArray<TObjectPtr<UPhysicsConstraintTemplate>> Constraints;
		TObjectPtr<UPhysicsAsset> TargetAsset;
	};
}