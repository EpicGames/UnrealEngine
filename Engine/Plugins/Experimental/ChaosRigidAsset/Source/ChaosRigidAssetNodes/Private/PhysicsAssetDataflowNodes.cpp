// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetDataflowNodes.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Misc/WildcardString.h"
#include "Dataflow/DataflowAttachment.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDrawObject.h"
#include "Animation/Skeleton.h"
#include "SkeletalDebugRendering.h"
#include "PhysicsAssetBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "ReferenceSkeleton.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsSimulation_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsActorHandle_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsJointHandle_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "Animation/AnimSequence.h"
#include "AnimationRuntime.h"
#include "AnimationUtils.h"
#include "ChaosVDRuntimeModule.h"

#define LOCTEXT_NAMESPACE "PhysicsAssetDataflowNodes"

FDataflowPhysicsAssetTerminalNode::FDataflowPhysicsAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid /*= FGuid::NewGuid()*/)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&State);
	RegisterInputConnection(&PhysicsAsset);
}

void FDataflowPhysicsAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	UPhysicsAsset* Target = Asset ? Cast<UPhysicsAsset>(Asset->GetOuter()) : nullptr;

	if(!Target)
	{
		Target = GetValue(Context, &PhysicsAsset);
	}

	if(Target)
	{
		const FPhysicsAssetDataflowState InState = GetValue(Context, &State);
		InState.DebugLog();

		UE::Chaos::RigidAsset::FPhysicsAssetBuilder::Make(InState.GetSkeleton(), InState.GetBodies().ToArray(), InState.Constraints.ToArray())
			.SetTargetAsset(Target)
			.Build();
	}
}

void FDataflowPhysicsAssetTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
}

void UE::Dataflow::RegisterPhysicsAssetTerminalNode()
{
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPhysicsAssetTerminalNode);
}

void UE::Dataflow::RegisterPhysicsAssetNodes()
{
	// Asset state
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPhysicsAssetMakeState);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPhysicsAssetAddBody);

	// Body Setup
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMakeBody);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSetBodyGeometry);

	// Joints
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMakeJoint);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAutoConstrainBodies);

	// Body Geometry
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAggGeomAddShape);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCreateGeometryForBones);

	// Bone Selections
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowNewBoneSelection);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAppendBoneSelection);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSelectBonesByName);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSelectConnectedBones);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMergeBoneSelection);

	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowRigidSimulationCachingNode);
	

	RegisterBoneGeometryGeneratorNodes();
	RegisterConstraintGeneratorNodes();
}

FDataflowPhysicsAssetAddBody::FDataflowPhysicsAssetAddBody(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid /*= FGuid::NewGuid()*/)
	: FRigidDataflowNode(InParam, InGuid)
{
	AddInputs(Body, State);
	AddPassthroughs(State);
}

void FDataflowPhysicsAssetAddBody::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&State))
	{
		USkeletalBodySetup* InBody = GetValue(Context, &Body);

		FPhysicsAssetDataflowState InState = GetValue(Context, &State);

		if(InBody && InBody->BoneName != NAME_None)
		{
			InState.SetBody(InBody->BoneName, InBody);
		}

		SetValue(Context, InState, &State);
	}
}

struct FDataflowDebugDrawBodySetupObject : public FDataflowDebugDrawBaseObject
{
	FDataflowDebugDrawBodySetupObject(IDataflowDebugDrawInterface::FDataflowElementsType& InDataflowElements, TObjectPtr<USkeletalBodySetup> InBody)
		: FDataflowDebugDrawBaseObject(InDataflowElements)
		, Body(InBody)
	{
	}

	virtual void PopulateDataflowElements() override
	{
	}

	virtual void DrawDataflowElements(FPrimitiveDrawInterface* PDI) override
	{
		if(!Body)
		{
			return;
		}

		const FKAggregateGeom& Geom = Body->AggGeom;
		const int32 NumGeoms = Geom.GetElementCount();

		for(int32 Index = 0; Index < NumGeoms; ++Index)
		{
			const FKShapeElem* Elem = Geom.GetElement(Index);
			Elem->DrawElemWire(PDI, Elem->GetTransform(), 1.0f, FColor::White);
		}
	}

	virtual FBox ComputeBoundingBox() const override
	{
		if(!Body)
		{
			return FBox(EForceInit::ForceInitToZero);
		}

		return Body->AggGeom.CalcAABB(FTransform::Identity);
	}

private:
	TObjectPtr<USkeletalBodySetup> Body;
};

struct FDataflowDebugDrawBoneSelectionObject : public FDataflowDebugDrawBaseObject
{
	FDataflowDebugDrawBoneSelectionObject(IDataflowDebugDrawInterface::FDataflowElementsType& InDataflowElements, FRigidAssetBoneSelection InSelection)
		: FDataflowDebugDrawBaseObject(InDataflowElements)
		, Selection(InSelection)
	{
	}

	virtual void PopulateDataflowElements() override
	{
	}

	virtual void DrawDataflowElements(FPrimitiveDrawInterface* PDI) override
	{
		if(!Selection.Mesh)
		{
			return;
		}

		FReferenceSkeleton& RefSkel = Selection.Mesh->GetRefSkeleton();

		const int32 NumBones = RefSkel.GetNum();
		TArray<FTransform> BoneTransforms;
		TArray<FBoneIndexType> RequiredBones;
		TArray<FLinearColor> BoneColors;
		TArray<int32> SelectedBones;
		TArray<TRefCountPtr<HHitProxy>> HitProxies;

		RefSkel.GetBoneAbsoluteTransforms(BoneTransforms);

		for(int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			RequiredBones.Add(BoneIndex);
		}

		for(const FRigidAssetBoneInfo& Bone : Selection.SelectedBones)
		{
			SelectedBones.Add(Bone.Index);
		}

		FSkelDebugDrawConfig DrawConfig;
		DrawConfig.bUseMultiColorAsDefaultColor = false;
		DrawConfig.BoneDrawMode = EBoneDrawMode::All;
		DrawConfig.BoneDrawSize = 1.f;
		DrawConfig.bAddHitProxy = false;
		DrawConfig.bForceDraw = true;
		DrawConfig.DefaultBoneColor = FLinearColor::Gray;
		DrawConfig.AffectedBoneColor = FLinearColor::Gray;
		DrawConfig.SelectedBoneColor = FLinearColor{ 1.0f, 0.75f, 0.1f };
		DrawConfig.ParentOfSelectedBoneColor = FLinearColor::Gray;

		SkeletalDebugRendering::DrawBones(
			PDI, FVector::Zero(),
			RequiredBones,
			RefSkel,
			BoneTransforms,
			SelectedBones,
			BoneColors,
			HitProxies,
			DrawConfig
		);
	}

	virtual FBox ComputeBoundingBox() const override
	{
		if(!Selection.Mesh)
		{
			return FBox(EForceInit::ForceInitToZero);
		}

		FReferenceSkeleton& RefSkel = Selection.Mesh->GetRefSkeleton();
		TArray<FTransform> BoneTransforms;

		RefSkel.GetBoneAbsoluteTransforms(BoneTransforms);

		FBox Result;

		for(const FTransform& Transform : BoneTransforms)
		{
			Result += Transform.GetTranslation();
		}

		return Result;
	}

private:
	FRigidAssetBoneSelection Selection;
};

struct FDataflowDebugDrawAggGeomObject : public FDataflowDebugDrawBaseObject
{
	FDataflowDebugDrawAggGeomObject(IDataflowDebugDrawInterface::FDataflowElementsType& InDataflowElements, const FKAggregateGeom* InGeom)
		: FDataflowDebugDrawBaseObject(InDataflowElements)
		, Geom(InGeom)
	{
	}

	void PopulateDataflowElements() override
	{
	}

	void DrawDataflowElements(FPrimitiveDrawInterface* PDI) override
	{
		if(!Geom)
		{
			return;
		}

		const int32 NumElems = Geom->GetElementCount();

		int32 ElementOffset = DataflowElements.Num();
		int32 ElementNum = NumElems;

		for(int32 Index = 0; Index < NumElems; ++Index)
		{
			const FKShapeElem* Elem = Geom->GetElement(Index);
			Elem->DrawElemWire(PDI, Elem->GetTransform(), 1.0f, FColor::Cyan);
		}
	}

	FBox ComputeBoundingBox() const override
	{
		if(!Geom)
		{
			return {};
		}

		return Geom->CalcAABB(FTransform::Identity);
	}

private:
	const FKAggregateGeom* Geom = nullptr;
};

#if WITH_EDITOR

bool FDataflowMakeBody::CanDebugDraw() const
{
	return true;
}

bool FDataflowMakeBody::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

void FDataflowMakeBody::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if(Body)
	{
		TRefCountPtr<IDataflowDebugDrawObject> BodyDrawObject(MakeDebugDrawObject<FDataflowDebugDrawBodySetupObject>(DataflowRenderingInterface.ModifyDataflowElements(), Body));
		DataflowRenderingInterface.DrawObject(BodyDrawObject);
	}
}
#endif

void FDataflowMakeBody::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Body))
	{
		TObjectPtr<USkeletalBodySetup> NewSetup = CastChecked<USkeletalBodySetup>(StaticDuplicateObject(Body, GetTransientPackage()));

		NewSetup->BoneName = GetValue(Context, &BoneName);

		SetValue(Context, NewSetup, &Body);
	}
}

void FDataflowMakeBody::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	// Construct the template object for this node
	Body = NewObject<USkeletalBodySetup>(GetOwner(), MakeUniqueObjectName(GetOwner(), USkeletalBodySetup::StaticClass(), "TemplateBody"));

	AddInputs(BoneName);
	AddOutputs(Body);
}

void FDataflowMakeJoint::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Joint))
	{
		TObjectPtr<UPhysicsConstraintTemplate> NewJoint = CastChecked<UPhysicsConstraintTemplate>(StaticDuplicateObject(Joint, GetTransientPackage()));

		SetValue(Context, NewJoint, &Joint);
	}
}

void FDataflowMakeJoint::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	// Construct the template object for this node
	Joint = NewObject<UPhysicsConstraintTemplate>(GetOwner());

	AddOutputs(Joint);
}

void FDataflowSelectBonesByName::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Selection))
	{
		FPhysicsAssetDataflowState InState = GetValue(Context, &State);

		TObjectPtr<USkeleton> Skeleton = InState.GetSkeleton();
		TObjectPtr<USkeletalMesh> Mesh = InState.GetMesh();

		if(!Skeleton)
		{
			Context.Error(TEXT("Attempted to make a selection without a Skeleton"), this);
			SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
			return;
		}

		if(!Mesh)
		{
			Context.Error(TEXT("Attempted to make a selection without a SkeletalMesh"), this);
			SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
			return;
		}

		const FWildcardString Pattern = SearchString;
		const FReferenceSkeleton RefSkel = Skeleton->GetReferenceSkeleton();

		FRigidAssetBoneSelection NewSelection;
		NewSelection.Skeleton = Skeleton;
		NewSelection.Mesh = Mesh;

		for(int32 BoneIndex = 0; BoneIndex < RefSkel.GetNum(); ++BoneIndex)
		{
			FName BoneName = RefSkel.GetBoneName(BoneIndex);

			if(Pattern.IsMatch(BoneName.ToString()))
			{
				NewSelection.SelectedBones.Emplace(BoneName, BoneIndex, RefSkel.GetDepthBetweenBones(BoneIndex, 0));
			}
		}

		NewSelection.SortBones();
		SetValue(Context, MoveTemp(NewSelection), &Selection);
	}
}

void FDataflowSelectBonesByName::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	AddInputs(State);
	AddOutputs(Selection);
}

void FDataflowAppendBoneSelection::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Selection))
	{
		const FRigidAssetBoneSelection& InA = GetValue(Context, &A);
		const FRigidAssetBoneSelection& InB = GetValue(Context, &B);

		if((InA.Skeleton != InB.Skeleton) || (InA.Mesh != InB.Mesh))
		{
			Context.Error(TEXT("Attmepted to append selections from multiple skeletons or meshes"), this);
			SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
			return;
		}

		FRigidAssetBoneSelection NewSelection = InA;

		for(const FRigidAssetBoneInfo& Bone : InB.SelectedBones)
		{
			NewSelection.SelectedBones.AddUnique(Bone);
		}

		NewSelection.SortBones();
		SetValue(Context, MoveTemp(NewSelection), &Selection);
	}
}

void FDataflowAppendBoneSelection::Register()
{
	AddInputs(A, B);
	AddOutputs(Selection);
}

void FDataflowSelectConnectedBones::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Selection))
	{
		const FRigidAssetBoneSelection& InSelection = GetValue(Context, &Selection);

		if(!InSelection.Skeleton)
		{
			Context.Error(TEXT("Attempted to make a selection without a Skeleton"), this);
			SafeForwardInput(Context, &Selection, &Selection);
			return;
		}

		const FReferenceSkeleton& RefSkel = InSelection.Skeleton->GetReferenceSkeleton();

		FRigidAssetBoneSelection NewSelection = InSelection;

		for(const FRigidAssetBoneInfo& Bone : InSelection.SelectedBones)
		{
			int32 CurrentIndex = Bone.Index;
			for(int32 UpCount = 0; UpCount < DistanceUp; ++UpCount)
			{
				int32 NewIndex = RefSkel.GetParentIndex(CurrentIndex);

				if(NewIndex == INDEX_NONE)
				{
					// Nothing more in this direction
					break;
				}

				if(!NewSelection.ContainsIndex(NewIndex))
				{
					NewSelection.SelectedBones.Emplace(RefSkel.GetBoneName(NewIndex), NewIndex, RefSkel.GetDepthBetweenBones(NewIndex, 0));
				}

				CurrentIndex = NewIndex;
			}

			int32 Level = 0;
			TArray<int32> BoneStack;
			BoneStack.Add(Bone.Index);

			while(Level < DistanceDown)
			{
				TArray<int32> NextBoneStack;
				while(BoneStack.Num() > 0)
				{
					const int32 Current = BoneStack.Pop(EAllowShrinking::No);

					TArray<int32> ChildBoneIndices;
					RefSkel.GetDirectChildBones(Current, ChildBoneIndices);

					for(const int32 ChildIndex : ChildBoneIndices)
					{
						if(!NewSelection.ContainsIndex(ChildIndex))
						{
							NewSelection.SelectedBones.Emplace(RefSkel.GetBoneName(ChildIndex), ChildIndex, RefSkel.GetDepthBetweenBones(ChildIndex, 0));
						}

						NextBoneStack.Push(ChildIndex);
					}
				}
				BoneStack = MoveTemp(NextBoneStack);

				++Level;
			}
		}

		NewSelection.SortBones();
		SetValue(Context, MoveTemp(NewSelection), &Selection);
	}
}

void FDataflowSelectConnectedBones::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	AddInputs(Selection);
	AddPassthroughs(Selection);
}

void FDataflowMergeBoneSelection::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Selection))
	{
		const FRigidAssetBoneSelection& InSelection = GetValue(Context, &Selection);

		if(!InSelection.Skeleton)
		{
			Context.Error(LOCTEXT("InvalidSelection_NoSkeleton", "Attempted to make a selection without a Skeleton"), this);
			SafeForwardInput(Context, &Selection, &Selection);
			return;
		}

		FRigidAssetBoneSelection NewSelection = MergeSelection(MergeProfile, InSelection);
		SetValue(Context, NewSelection, &Selection);
	}
}

void FDataflowMergeBoneSelection::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	AddInputs(Selection);
	AddPassthroughs(Selection);
}

#if WITH_EDITOR
bool FDataflowCreateGeometryForBones::CanDebugDraw() const
{
	return true;
}

bool FDataflowCreateGeometryForBones::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

void FDataflowCreateGeometryForBones::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	const FRigidAssetBoneSelection& InSelection = GetValue(Context, &Selection);
	TRefCountPtr<IDataflowDebugDrawObject> DrawObject(MakeDebugDrawObject<FDataflowDebugDrawBoneSelectionObject>(DataflowRenderingInterface.ModifyDataflowElements(), InSelection));
	DataflowRenderingInterface.DrawObject(DrawObject);

	TObjectPtr<UBoneGeometryGenerator> InGenerator = GetValue(Context, &Generator);

	if(InGenerator)
	{
		InGenerator->DebugDraw(Context, DataflowRenderingInterface, DebugDrawParameters, InSelection);
	}
}
#endif

void FDataflowCreateGeometryForBones::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&State))
	{
		TObjectPtr<UBoneGeometryGenerator> InGenerator = GetValue(Context, &Generator);

		if(!InGenerator)
		{
			Context.Warning(TEXT("FDataflowCreateGeometryForBones: No method selected for body generation"), this);
			SafeForwardInput(Context, &State, &State);
			return;
		}

		const FRigidAssetBoneSelection& InSelection = GetValue(Context, &Selection);

		if(!InSelection.Skeleton || !InSelection.Mesh)
		{
			Context.Error(TEXT("Attempted to make geometry without a skeleton or mesh"), this);
			SafeForwardInput(Context, &State, &State);
			return;
		}

		TObjectPtr<USkeletalBodySetup> InTemplate = GetValue(Context, &TemplateBody);
		if(!InTemplate)
		{
			Context.Error(TEXT("Attempted to make a body without a template body setup"), this);
			SafeForwardInput(Context, &State, &State);
			return;
		}

		FPhysicsAssetDataflowState InState = GetValue(Context, &State);

		if(!InState.HasData())
		{
			Context.Error(TEXT("Attempted to make a body without a valid asset state"), this);
			SafeForwardInput(Context, &State, &State);
			return;
		}

		// Invoke the geometry generator
		TArray<FRigidAssetBoneGeometry> Geoms = InGenerator->Build(InSelection);

		// Create new bodies for the geometry - or append new geometries to existing bodies if they already exist
		for(int32 Index = 0; Index < Geoms.Num(); ++Index)
		{
			const UE::Chaos::RigidAsset::FSimpleGeometry& Geom = Geoms[Index].Geometry;
			const FRigidAssetBoneInfo& Bone = Geoms[Index].Bone;

			TObjectPtr<USkeletalBodySetup> TemplateToUse = InTemplate;
			if(!TemplateToUse)
			{
				TemplateToUse = USkeletalBodySetup::StaticClass()->GetDefaultObject<USkeletalBodySetup>();
			}

			USkeletalBodySetup* BoneSetup = InState.FindOrCreateBodyChecked(Bone.Name, TemplateToUse);
			
			if(!Geom.GeometryElement)
			{
				Context.Error(TEXT("Generator failed to create usable geometry"), this);
				SafeForwardInput(Context, &State, &State);
				return;
			}

			BoneSetup->AggGeom.VisitShapeAndContainer(*Geom.GeometryElement.Get(), [] <typename ElemType> (const ElemType & ElemTyped, TArray<ElemType>&Container)
			{
				Container.Add(ElemTyped);
			});
		}

		SetValue(Context, MoveTemp(InState), &State);
	}
}

void FDataflowCreateGeometryForBones::Register()
{
	AddInputs(Generator, TemplateBody, State, Selection);
	AddPassthroughs(State);
}

void FDataflowAutoConstrainBodies::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	TObjectPtr<UConstraintGenerator> InGenerator = GetValue(Context, &Generator);

	if(!InGenerator)
	{
		Context.Warning(TEXT("FDataflowAutoConstrainBodies: No method selected for constraint generation"), this);
		SafeForwardInput(Context, &State, &State);
		return;
	}

	const FRigidAssetBoneSelection& InSelection = GetValue(Context, &Selection);

	if(!InSelection.Skeleton || !InSelection.Mesh)
	{
		Context.Error(TEXT("Attempted to make constraints without a skeleton or mesh"), this);
		SafeForwardInput(Context, &State, &State);
		return;
	}

	TObjectPtr<UPhysicsConstraintTemplate> InTemplate = GetValue(Context, &TemplateConstraint);

	if(!InTemplate)
	{
		Context.Error(TEXT("Attempted to make a constraint without a template"), this);
		SafeForwardInput(Context, &State, &State);
		return;
	}

	FPhysicsAssetDataflowState InState = GetValue(Context, &State);
	TArray<TObjectPtr<UPhysicsConstraintTemplate>> GeneratedConstraints = InGenerator->Build(InTemplate, InSelection);

	InState.Constraints.Append(GeneratedConstraints);

	SetValue(Context, MoveTemp(InState), &State);
}

void FDataflowAutoConstrainBodies::Register()
{
	AddInputs(Generator, TemplateConstraint, State, Selection);
	AddPassthroughs(State);
}

#if WITH_EDITOR
bool FDataflowSetBodyGeometry::CanDebugDraw() const
{
	return true;
}

bool FDataflowSetBodyGeometry::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

void FDataflowSetBodyGeometry::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if(DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned)
	{
		if(Body)
		{
			TRefCountPtr<IDataflowDebugDrawObject> BodyDrawObject(MakeDebugDrawObject<FDataflowDebugDrawBodySetupObject>(DataflowRenderingInterface.ModifyDataflowElements(), Body));
			DataflowRenderingInterface.DrawObject(BodyDrawObject);
		}
	}
}
#endif

void FDataflowSetBodyGeometry::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Body))
	{
		TObjectPtr<USkeletalBodySetup> InSetup = GetValue(Context, &Body);

		check(InSetup);

		TObjectPtr<USkeletalBodySetup> NewSetup = CastChecked<USkeletalBodySetup>(StaticDuplicateObject(InSetup, GetTransientPackage()));

		NewSetup->AggGeom = GetValue(Context, &Geometry);

		SetValue(Context, NewSetup, &Body);
	}
}

void FDataflowSetBodyGeometry::Register()
{
	AddInputs(Body, Geometry);
	AddPassthroughs(Body);
}

void FDataflowNewBoneSelection::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&Selection))
	{
		TObjectPtr<USkeleton> InSkeleton = GetValue(Context, &Skeleton);
		TObjectPtr<USkeletalMesh> InMesh = GetValue(Context, &Mesh);

		// Unless overridden, get the skeleton from the currently edited asset
		if(!InSkeleton)
		{
			if(TObjectPtr<UPhysicsAsset> PhysAsset = GetAttachmentOwnerAs<UPhysicsAsset>(Context))
			{
				InSkeleton = PhysAsset->GetPreviewMesh() ? PhysAsset->GetPreviewMesh()->GetSkeleton() : nullptr;
			}
			else
			{
				Context.Error(TEXT("Attempted to make a selection without a Skeleton"), this);
				SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
				return;
			}
		}

		// Unless overridden, pull the skeletal mesh from the selected skeleton
		if(!InMesh)
		{
			if(InSkeleton)
			{
				InMesh = InSkeleton->GetPreviewMesh();
			}
			else
			{
				Context.Error(TEXT("Attempted to make a selection without a Skeletal Mesh"), this);
				SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
				return;
			}
		}

		if(!InMesh || !InSkeleton)
		{
			Context.Error(TEXT("Unable to identify a mesh or skeleton"), this);
			SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
			return;
		}

		if(InSkeleton != InMesh->GetSkeleton())
		{
			Context.Error(TEXT("Specified mesh is not compatible with the specified skeleton"), this);
			SetValue(Context, FRigidAssetBoneSelection{}, &Selection);
			return;
		}

		FRigidAssetBoneSelection NewSelection;
		NewSelection.Skeleton = InSkeleton;
		NewSelection.Mesh = InMesh;

		SetValue(Context, MoveTemp(NewSelection), &Selection);
	}
}

void FDataflowNewBoneSelection::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	AddInputs(Skeleton, Mesh);
	AddOutputs(Selection);
}

void FDataflowPhysicsAssetMakeState::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if(Out->IsA(&State))
	{
		TObjectPtr<USkeletalMesh> InTarget = GetValue(Context, &TargetMesh);

		// If there's no override - pick from the context asset
		if(!InTarget)
		{
			if(TObjectPtr<UPhysicsAsset> PhysAsset = GetAttachmentOwnerAs<UPhysicsAsset>(Context))
			{
				InTarget = PhysAsset->GetPreviewMesh();

				if(!InTarget)
				{
					Context.Error(TEXT("Attempted to make a state without a valid mesh"), this);
					SetValue(Context, FPhysicsAssetDataflowState{}, &State);
					return;
				}
			}
			else
			{
				Context.Error(TEXT("Attempted to make a state without a valid skeleton"), this);
				SetValue(Context, FPhysicsAssetDataflowState{}, &State);
				return;
			}
		}

		// Make sure we have initialised the mesh in order to get vertex data
		InTarget->CalculateInvRefMatrices();

		SetValue(Context, FPhysicsAssetDataflowState{ InTarget->GetSkeleton(), InTarget }, &State);
	}
}

void FDataflowPhysicsAssetMakeState::Register()
{
	AddInputs(TargetMesh);
	AddOutputs(State);
}

#if WITH_EDITOR
bool FDataflowAggGeomAddShape::CanDebugDraw() const
{
	return true;
}

bool FDataflowAggGeomAddShape::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

void FDataflowAggGeomAddShape::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	TRefCountPtr<IDataflowDebugDrawObject> Obj(MakeDebugDrawObject<FDataflowDebugDrawAggGeomObject>(DataflowRenderingInterface.ModifyDataflowElements(), &AggGeom));
	DataflowRenderingInterface.DrawObject(Obj);
}
#endif

void FDataflowAggGeomAddShape::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::RigidAsset;

	if(Out->IsA(&AggGeom))
	{
		const int32 NumShapeInputs = Shapes.Num();

		const FKAggregateGeom& InGeom = GetValue(Context, &AggGeom);

		FKAggregateGeom NewAggGeom = InGeom;

		for(int32 Index = 0; Index < NumShapeInputs; ++Index)
		{
			const FSimpleGeometry& InShape = GetValue(Context, &Shapes[Index]);

			if(InShape.GeometryElement)
			{
				NewAggGeom.AddElement(*InShape.GeometryElement.Get());
			}
		}

		SetValue(Context, MoveTemp(NewAggGeom), &AggGeom);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowRigidSimulationCachingNode::Register(const UE::Dataflow::FNodeParameters& InParam)
{
	AddInputs(State, Duration, FrameRate, Gravity, Velocity, Affector);
	AddOutputs(Animation);
}

void FDataflowRigidSimulationCachingNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace ImmediatePhysics;

	if (Out->IsA(&Animation))
	{
		const FPhysicsAssetDataflowState& InState = GetValue(Context, &State);
		const float InDuration= GetValue(Context, &Duration);
		int32 InFrameRate = GetValue(Context, &FrameRate);
		if (InFrameRate <= 0)
		{
			InFrameRate = 30;
		}
		const FVector InGravity= GetValue(Context, &Gravity);
		const FVector InVelocity = GetValue(Context, &Velocity);
		
		const FConstStructView AffectorStructView = GetValue(Context, &Affector).GetConstStructView();
		const FDataflowSimulationAffector* InAffector = AffectorStructView.GetPtr<FDataflowSimulationAffector>();

		const float FixedTimeStep = 1.0f / static_cast<float>(InFrameRate);
		const int32 NumFrames = FMath::Max(1, (int32)(InDuration * InFrameRate));
		
		TObjectPtr<UAnimSequence> OutAnim = NewObject<UAnimSequence>();
#if WITH_EDITOR
		if (InState.HasData())
		{
			OutAnim->SetSkeleton(InState.GetSkeleton());
			OutAnim->BoneCompressionSettings = FAnimationUtils::GetDefaultAnimationRecorderBoneCompressionSettings();

			IAnimationDataController& Controller = OutAnim->GetController();
			Controller.SetModel(OutAnim->GetDataModelInterface());
			Controller.InitializeModel();
			Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float);
			Controller.RemoveAllBoneTracks(/*bTransactRecording*/false);
			OutAnim->ResetAnimation();

			//Set Interpolation type (Step or Linear), doesn't look like there is a controller for this.
			OutAnim->Interpolation = EAnimInterpolationType::Linear;
			const FFrameRate RecordingRate = FFrameRate(InFrameRate, 1);
			// Set frame rate and number of frames
			Controller.SetFrameRate(RecordingRate, /*bTransactRecording*/false);
			Controller.SetNumberOfFrames(NumFrames, /*bTransactRecording*/false);

			OutAnim->RetargetSource = InState.GetSkeleton()->GetRetargetSourceForMesh(InState.GetMesh());

			FSimulationState SimState;
			SimState.FixedTimeStep = FixedTimeStep;
			BuildSimulationFromState(InState, SimState);
			if (SimState.Simulation.IsValid())
			{
				TMap<FName, FRawAnimSequenceTrack> RawTracksByName;
				TMap<FName, FActorHandle*> ParentActorByChildName;

				const FReferenceSkeleton& RefSkeleton = InState.GetMesh()->GetRefSkeleton();

				// create the tracks from the simulation actors
				for (const TPair<FName, FActorHandle*>& ActorByName : SimState.ActorsByName)
				{
					if (ActorByName.Value)
					{
						const FName ActorName = ActorByName.Key;
						Controller.AddBoneCurve(ActorName, /*bTransactRecording*/false);
						RawTracksByName.Add(ActorName, FRawAnimSequenceTrack());

						const int32 BoneIndex = RefSkeleton.FindBoneIndex(ActorName);
						const int32 ParentBoneIndex = (BoneIndex == INDEX_NONE) ? INDEX_NONE : RefSkeleton.GetParentIndex(BoneIndex);
						if (ParentBoneIndex != INDEX_NONE)
						{
							const FName ParentName = RefSkeleton.GetBoneName(ParentBoneIndex);
							if (FActorHandle* ParentHandle = SimState.ActorsByName.FindRef(ParentName))
							{
								ParentActorByChildName.Add(ActorName, ParentHandle);
							}
						}
					}
				}

				for (int32 Frame = 0; Frame < NumFrames; ++Frame)
				{
					// record results
					for (const TPair<FName, FActorHandle*>& ActorByName : SimState.ActorsByName)
					{
						if (FActorHandle* Handle = ActorByName.Value)
						{
							const FName ActorName = ActorByName.Key;

							FTransform ActorLocalTransform = Handle->GetWorldTransform();
							if (const FActorHandle* ParentHandle = ParentActorByChildName.FindRef(ActorName))
							{
								const FTransform ParentActorTransform = ParentHandle->GetWorldTransform();
								ActorLocalTransform.SetToRelativeTransform(ParentActorTransform);
							}

							if (FRawAnimSequenceTrack* RawTrack = RawTracksByName.Find(ActorName))
							{
								RawTrack->PosKeys.Add(FVector3f(ActorLocalTransform.GetTranslation()));
								RawTrack->RotKeys.Add(FQuat4f(ActorLocalTransform.GetRotation()));
								RawTrack->ScaleKeys.Add(FVector3f(ActorLocalTransform.GetScale3D()));
							}
						}
					}

					// apply affectors
					if (InAffector)
					{
						InAffector->Process(SimState.Actors);
					}

					SimState.Simulation->UpdateSimulationSpace(FTransform::Identity, InVelocity, FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector);
					RunSimulation(SimState, InGravity);
				}

				// done let's save the tracks 
				for (const TPair<FName, FRawAnimSequenceTrack>& TrackByName : RawTracksByName)
				{
					const FRawAnimSequenceTrack& RawTrack = TrackByName.Value;
					const FName TrackName = TrackByName.Key;
					Controller.SetBoneTrackKeys(TrackName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, /*bTransactRecording*/false);
				}

				OutAnim->PostEditChange();
			}
		}
		else
		{
			Context.Warning("Input State TargetSkeleton or TargetMesh is null, outputing an empty animation sequence", this, Out);
		}
#endif // WITH_EDITOR
		SetValue(Context, OutAnim, &Animation);
	}
}
void FDataflowRigidSimulationCachingNode::RunSimulation(FSimulationState& SimulationState, const FVector& InGravity) const
{
	if (SimulationState.Simulation)
	{
		SimulationState.Simulation->Simulate_AssumesLocked(SimulationState.FixedTimeStep, SimulationState.FixedTimeStep, 1, InGravity);
	}
}

void FDataflowRigidSimulationCachingNode::BuildSimulationFromState(const FPhysicsAssetDataflowState& InState, FSimulationState& OutSimState) const
{
	using namespace ImmediatePhysics;

	if (InState.GetMesh() == nullptr)
	{
		return;
	}

	OutSimState.Simulation = MakeUnique<FSimulation>();
#if WITH_CHAOS_VISUAL_DEBUGGER
	OutSimState.Simulation->GetChaosVDContextData().Id = FChaosVDRuntimeModule::Get().GenerateUniqueID();
	OutSimState.Simulation->GetChaosVDContextData().Type = static_cast<int32>(EChaosVDContextType::Solver);
#endif

	// structure to efficiently compute bones component space
	TArray<FTransform> CachedTransforms;
	TArray<bool> CachedTransformReady;
	const FReferenceSkeleton& RefSkeleton = InState.GetMesh()->GetRefSkeleton();
	CachedTransforms.SetNumUninitialized(RefSkeleton.GetRefBonePose().Num());
	CachedTransformReady.SetNumZeroed(RefSkeleton.GetRefBonePose().Num());

	for (TObjectPtr<USkeletalBodySetup> BodySetup : InState.GetBodies())
	{
		TUniquePtr<FBodyInstance> BodyInstance = CreateBodyInstance(BodySetup.Get());
		if (BodyInstance)
		{
			const int32 BoneIndex = RefSkeleton.FindBoneIndex(BodySetup->BoneName);
			FTransform BoneComponentSpaceTransform{ FTransform::Identity };
			if (BoneIndex != INDEX_NONE)
			{
				BoneComponentSpaceTransform = FAnimationRuntime::GetComponentSpaceTransformWithCache(RefSkeleton, RefSkeleton.GetRefBonePose(), BoneIndex, CachedTransforms, CachedTransformReady);
			}

			const bool bSimulated = (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated || BodySetup->PhysicsType == EPhysicsType::PhysType_Default);
			EActorType ActorType = bSimulated ? EActorType::DynamicActor : EActorType::KinematicActor;
			FActorHandle* NewBodyHandle = OutSimState.Simulation->CreateActor(MakeActorSetup(ActorType, BodyInstance.Get(), BodyInstance->GetUnrealWorldTransform()));
			if (NewBodyHandle)
			{
				// This is a local simulation we assume that world space is the same as component space
				NewBodyHandle->InitWorldTransform(BoneComponentSpaceTransform);
				NewBodyHandle->SetName(BodySetup->BoneName);
				NewBodyHandle->SetEnabled(true);
				NewBodyHandle->SetHasCollision(false);
				//OutSimState.Simulation->AddToCollidingPairs(NewBodyHandle);
				OutSimState.ActorsByName.Add(BodySetup->BoneName, NewBodyHandle);
				OutSimState.Actors.Add(NewBodyHandle);
				UE_LOGF(LogTemp, Warning, "@@ Create Sim Actor : [%ls] - bSimulated[%d] - Transform[%ls]", *BodySetup->BoneName.ToString(), (int)bSimulated, *BoneComponentSpaceTransform.ToString());
			}
		}
	}


	TArray<FSimulation::FIgnorePair> IgnoreCollisionPairs;
	for (TObjectPtr<UPhysicsConstraintTemplate> ConstraintTemplate: InState.Constraints)
	{
		if (ConstraintTemplate)
		{
			FActorHandle* Actor1 = OutSimState.ActorsByName.FindRef(ConstraintTemplate->DefaultInstance.ConstraintBone1, nullptr);
			FActorHandle* Actor2 = OutSimState.ActorsByName.FindRef(ConstraintTemplate->DefaultInstance.ConstraintBone2, nullptr);
			if (Actor1 && Actor2)
			{
				OutSimState.Simulation->CreateJoint(MakeJointSetup(&ConstraintTemplate->DefaultInstance, Actor1, Actor2));
				FSimulation::FIgnorePair Pair;
				Pair.A = Actor1;
				Pair.B = Actor2;
				IgnoreCollisionPairs.Add(Pair);

			}
		}
	}

	OutSimState.Simulation->SetIgnoreCollisionPairTable(IgnoreCollisionPairs);
	//OutSimState.Simulation->SetIgnoreCollisionActors(IgnoreCollisionActors);
	OutSimState.Simulation->SetSolverSettings(
		OutSimState.FixedTimeStep,
		/*CullDistance*/ 10000000,
		/*MaxDepenetrationVelocity*/ 100000000,
		/*bUseLinearJointSolver*/ true,
		/*PositionIterations*/ 8,
		/*VelocityIterations*/ 2,
		/*SolverSettings.*/ 1,
		/*bUseManifolds*/ true);
}

TUniquePtr<FBodyInstance> FDataflowRigidSimulationCachingNode::CreateBodyInstance(USkeletalBodySetup* BodySetup) const
{
	TUniquePtr<FBodyInstance> BodyInstance;
	if (BodySetup && BodySetup->AggGeom.GetElementCount() > 0)
	{
		BodyInstance = MakeUnique<FBodyInstance>();
		BodyInstance->bAutoWeld = false;
		BodyInstance->bSimulatePhysics = false;
		BodyInstance->InitBody(BodySetup, FTransform::Identity, nullptr, nullptr);

		if (!BodyInstance->IsValidBodyInstance() || !ensure(BodyInstance->GetBodySetup()))
		{
			// Some part of the initialization process failed.
			BodyInstance.Reset();
		}
	}
	return BodyInstance;
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowAggGeomAddShape::Register()
{
	AddInputs(AggGeom);
	AddPassthroughs(AggGeom);
}

#undef LOCTEXT_NAMESPACE
