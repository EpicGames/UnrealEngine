// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/ConvertibleTo.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "Dataflow/DataflowTerminalNode.h"

#include "PhysicsAssetDataflowState.h"
#include "Generators/BoneGeometryGenerators.h"
#include "Generators/ConstraintGenerators.h"
#include "Generators/SimulationAffector.h"
#include "BoneSelection.h"
#include "RigidDataflowNode.h"
#include "ShapeElemNodes.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"

#include "PhysicsAssetDataflowNodes.generated.h"

class USkeleton;
class USkeletalMesh;
class UPhysicsAsset;
class USkeletalBodySetup;
class UPhysicsConstraintTemplate;
class UAnimSequence;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * The asset state is the object that holds the intermediate data that is built throughout the execution of the
 * dataflow graph. It's analogous to a collection used in other dataflow graphs. This graph doesn't use a
 * collection due to needing support for copy-on-write with UObjects
 * Asset state objects are lightweight and should be passed by-value as they support copy-on-write so each
 * state instance is fairly small and references its shared state
 */
USTRUCT()
struct FDataflowPhysicsAssetMakeState : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPhysicsAssetMakeState, "PhysicsAssetMakeState", "PhysicsAsset", "")

public:

	FDataflowPhysicsAssetMakeState(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	// Optional mesh input if creating a state for a mesh that isn't currently being edited
	UPROPERTY(EditAnywhere, Category = Config, meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> TargetMesh;

	// Resulting state for the mesh
	UPROPERTY(meta = (DataflowOutput))
	FPhysicsAssetDataflowState State;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Add a shape to an aggregate geometry
 */
USTRUCT()
struct FDataflowAggGeomAddShape : public FRigidDataflowMultiInputNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAggGeomAddShape, "AggGeomAddShape", "PhysicsAsset", "")
	DATAFLOW_NODE_RENDER_TYPE("GeomRender", FName("FKAggregateGeom"), "AggGeom")

public:

	FDataflowAggGeomAddShape(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowMultiInputNode(Shapes, InParam, InGuid) // -V1050
	{
		InitPins();
		Register();
	}

private:

	// Shapes to add to the aggregate geometry
	UPROPERTY()
	TArray<UE::Chaos::RigidAsset::FSimpleGeometry> Shapes;

	// The aggregate geometry to add the shape to
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = AggGeom))
	FKAggregateGeom AggGeom;

#if WITH_EDITOR
	bool CanDebugDraw() const override;
	bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif


	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

template<>
struct TStructOpsTypeTraits<FDataflowAggGeomAddShape> : public TStructOpsTypeTraitsBase2<FDataflowAggGeomAddShape>
{
	enum
	{
		WithCopy = false
	};
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Terminal node that converts an asset state into the final asset
 */
USTRUCT(meta = (DataflowTerminal))
struct FDataflowPhysicsAssetTerminalNode : public FDataflowTerminalNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPhysicsAssetTerminalNode, "PhysicsAssetTerminal", "Terminal", "")

public:

	FDataflowPhysicsAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	// The state to convert into the final asset
	UPROPERTY(meta = (DataflowInput))
	FPhysicsAssetDataflowState State;

	// Override target asset to write to - optional, defaults to the current asset in the editor
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<UPhysicsAsset> PhysicsAsset = nullptr;

	// FDataflowTerminalNode
	void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	void Evaluate(UE::Dataflow::FContext& Context) const override;
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Add a body setup to an asset state
 */
USTRUCT()
struct FDataflowPhysicsAssetAddBody : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPhysicsAssetAddBody, "PhysicsAssetAddBody", "PhysicsAsset", "")

public:

	FDataflowPhysicsAssetAddBody(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	// Body to add
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<USkeletalBodySetup> Body;

	// State to write to
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = State))
	FPhysicsAssetDataflowState State;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Make a single body setup from a template
 */
USTRUCT()
struct FDataflowMakeBody : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMakeBody, "Make Body Setup", "PhysicsAsset", "")

public:

	FDataflowMakeBody(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

#if WITH_EDITOR
	bool CanDebugDraw() const override;
	bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif


	// Template body setup for details panel and output new body instance
	UPROPERTY(EditAnywhere, Instanced, Category = Body, meta = (DataflowOutput))
	TObjectPtr<USkeletalBodySetup> Body;

	// Bone that this body should be bound to
	UPROPERTY(EditAnywhere, Category = Body, meta = (DataflowInput))
	FName BoneName;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Make a single joint setup from a template
 */
USTRUCT()
struct FDataflowMakeJoint : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMakeJoint, "Make Joint", "PhysicsAsset", "")

public:

	FDataflowMakeJoint(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

	UPROPERTY(EditAnywhere, Instanced, Category = Body, meta = (DataflowOutput))
	TObjectPtr<UPhysicsConstraintTemplate> Joint;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Select bones in a mesh given a name search pattern
 */
USTRUCT()
struct FDataflowSelectBonesByName : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSelectBonesByName, "Select Bones by Name", "PhysicsAsset", "")
	DATAFLOW_NODE_RENDER_TYPE("GeomRender", FName("FBoneSelection"), "Selection")

public:

	FDataflowSelectBonesByName(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

	// State to select bones within
	UPROPERTY(meta = (DataflowInput))
	FPhysicsAssetDataflowState State;

	// Resulting selection
	UPROPERTY(meta = (DataflowOutput))
	FRigidAssetBoneSelection Selection;

	// Search pattern to use. Supports wildcards (? - match single character, * - match sequence)
	UPROPERTY(EditAnywhere, Category = Parameters)
	FString SearchString;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Append a bone selection to another
 */
USTRUCT()
struct FDataflowAppendBoneSelection : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAppendBoneSelection, "Append Selected Bones", "PhysicsAsset", "")
	DATAFLOW_NODE_RENDER_TYPE("GeomRender", FName("FBoneSelection"), "Selection")

public:

	FDataflowAppendBoneSelection(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	// First selection
	UPROPERTY(meta = (DataflowInput))
	FRigidAssetBoneSelection A;

	// Second selection
	UPROPERTY(meta = (DataflowInput))
	FRigidAssetBoneSelection B;

	// Resulting selection, a combination of A and B
	UPROPERTY(meta = (DataflowOutput))
	FRigidAssetBoneSelection Selection;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Select bones connected to each bone already in a selection.
 */
USTRUCT()
struct FDataflowSelectConnectedBones : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSelectConnectedBones, "Select Connected Bones", "PhysicsAsset", "")
	DATAFLOW_NODE_RENDER_TYPE("GeomRender", FName("FBoneSelection"), "Selection")

public:

	FDataflowSelectConnectedBones(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

	// Disance to walk up the hierarchy to find bones
	UPROPERTY(EditAnywhere, Category = Settings)
	int32 DistanceUp = 0;

	// Distance to walk down the hierarchy to find bones
	UPROPERTY(EditAnywhere, Category = Settings)
	int32 DistanceDown = 0;

	// Resulting bone selction
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Selection"))
	FRigidAssetBoneSelection Selection;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Merge a bone selection using the specified merge profile
 */
USTRUCT()
struct FDataflowMergeBoneSelection : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMergeBoneSelection, "Merge Bones in Selection", "PhysicsAsset", "")
	DATAFLOW_NODE_RENDER_TYPE("GeomRender", FName("FBoneSelection"), "Selection")

public:

	FDataflowMergeBoneSelection(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

	// Resulting bone selection
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Selection"))
	FRigidAssetBoneSelection Selection;

	// Merge settings to use to decide how to merge the bones
	UPROPERTY(EditAnywhere, Category = Settings)
	FBaseGenerationSettings MergeProfile;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Apply a geometry generator to a bone selection, creating bodies for each bone
 * The method of geometry generation is generic and offloaded to a "Generator" object/
 * @see UBoneGeometryGenerator for implementation and creating your own generator node.
 */
USTRUCT()
struct FDataflowCreateGeometryForBones : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCreateGeometryForBones, "Create Geometry for Bones", "PhysicsAsset", "")
	DATAFLOW_NODE_RENDER_TYPE("GeomRender", FName("FPhysicsAssetDataflowState"), "State")

public:

	FDataflowCreateGeometryForBones(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	UPROPERTY(meta=(DataflowInput))
	TObjectPtr<UBoneGeometryGenerator> Generator;


	UPROPERTY(meta=(DataflowInput))
	TObjectPtr<USkeletalBodySetup> TemplateBody;

	UPROPERTY(EditAnywhere, Category=Generation)
	bool bCreateMissingBodies = true;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "State"))
	FPhysicsAssetDataflowState State;

	UPROPERTY(meta = (DataflowInput))
	FRigidAssetBoneSelection Selection;

#if WITH_EDITOR
	bool CanDebugDraw() const override;
	bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Apply a constraint generator to a bone selection, creating constraints for each pair of bodies
 * The method of constraint generation is generic and offloaded to a "Generator" object/
 * @see UConstraintGenerator for implementation and creating your own generator node.
 */
USTRUCT()
struct FDataflowAutoConstrainBodies : public FRigidDataflowNode
{
	GENERATED_BODY();
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAutoConstrainBodies, "Create Constraints for Bones", "PhysicsAsset", "")

public:

	FDataflowAutoConstrainBodies(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	// Generation method to use
	UPROPERTY(meta=(DataflowInput))
	TObjectPtr<UConstraintGenerator> Generator;

	// Template constraint to take properties from
	UPROPERTY(meta=(DataflowInput))
	TObjectPtr<UPhysicsConstraintTemplate> TemplateConstraint;

	// State to apply the joints to
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "State"))
	FPhysicsAssetDataflowState State;

	// Optional selection, restricts the bodies that will attempt to auto constrain
	UPROPERTY(meta = (DataflowInput))
	FRigidAssetBoneSelection Selection;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Given a body setup, replace the internal aggregate geometry with the provided geometry
 */
USTRUCT()
struct FDataflowSetBodyGeometry : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSetBodyGeometry, "Set Body Geometry", "PhysicsAsset", "")

public:

	FDataflowSetBodyGeometry(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register();
	}

private:

	// The body to modify
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Body"))
	TObjectPtr<USkeletalBodySetup> Body;

	// The Geometry to set on the body
	UPROPERTY(meta = (DataflowInput))
	FKAggregateGeom Geometry;

#if WITH_EDITOR
	bool CanDebugDraw() const override;
	bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Create a new, empty selection for the specified mesh
 */
USTRUCT()
struct FDataflowNewBoneSelection : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowNewBoneSelection, "New Bone Selection", "PhysicsAsset", "")

public:

	FDataflowNewBoneSelection(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

	// The created selection
	UPROPERTY(meta = (DataflowOutput))
	FRigidAssetBoneSelection Selection;

	// The skeleton to target if not using the currently edited asset
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<USkeleton> Skeleton = nullptr;

	// The Mesh to target if not using the currently edited asset
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> Mesh = nullptr;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
* Run a rigid simulation of a Physics asset state and bake it into an animation file 
*/
USTRUCT(meta = (Experimental))
struct FDataflowRigidSimulationCachingNode : public FRigidDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowRigidSimulationCachingNode, "Rigid Simulation Caching", "PhysicsAsset", "")

public:

	FDataflowRigidSimulationCachingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FRigidDataflowNode(InParam, InGuid)
	{
		Register(InParam);
	}

private:

	/** Input physics asset state to use for the simulation */
	UPROPERTY(meta = (DataflowInput))
	FPhysicsAssetDataflowState State;

	/** Output animation asset  */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UAnimSequence> Animation = nullptr;

	/** Duration in seconds of the simulation */
	UPROPERTY(EditAnywhere, Category = Caching, meta = (DataflowInput, ClampMin=0))
	float Duration = 5.0;

	/** Number of frames per second */
	UPROPERTY(EditAnywhere, Category = Caching, meta = (DataflowInput, ClampMin = 1))
	int32 FrameRate = 30;

	/** Gravity vector in cm/s^2 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (DataflowInput))
	FVector Gravity = FVector(0, 0, 980);

	/** Simulation space velocity in cm/s */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (DataflowInput))
	FVector Velocity = FVector(0, 0, 0);

	/** Affector used to manipulate objects in the simulation */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (DataflowInput))
	FDataflowSimulationAffectorTypes Affector;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void Register(const UE::Dataflow::FNodeParameters& InParam);

	TUniquePtr<FBodyInstance> CreateBodyInstance(USkeletalBodySetup* BodySetup) const;

	struct FSimulationState
	{
		float FixedTimeStep = 0;
		TUniquePtr<ImmediatePhysics::FSimulation> Simulation;
		TMap<FName, ImmediatePhysics::FActorHandle*> ActorsByName;
		TArray<ImmediatePhysics::FActorHandle*> Actors;
	};

	void BuildSimulationFromState(const FPhysicsAssetDataflowState& InState, FSimulationState& OutSimState) const;
	void RunSimulation(FSimulationState& Simulation, const FVector& InGravity) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterPhysicsAssetTerminalNode();
	void RegisterPhysicsAssetNodes();
}
