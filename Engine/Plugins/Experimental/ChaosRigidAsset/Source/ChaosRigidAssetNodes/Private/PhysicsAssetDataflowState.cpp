// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetDataflowState.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "Animation/Skeleton.h"
#include "UObject/Package.h"

FPhysicsAssetDataflowState::FPhysicsAssetDataflowState() = default;

FPhysicsAssetDataflowState::FPhysicsAssetDataflowState(TObjectPtr<USkeleton> InTargetSkeleton, TObjectPtr<USkeletalMesh> InTargetMesh)
	: TargetSkeleton(InTargetSkeleton)
	, TargetMesh(InTargetMesh)
{
	if(HasData())
	{
		const int32 NumBones = TargetSkeleton->GetReferenceSkeleton().GetNum();
		BoneIndexToBody.SetNum(NumBones);

		for(int32 Index = 0; Index < NumBones; ++Index)
		{
			BoneIndexToBody.Modify(Index, INDEX_NONE);
		}
	}
}

USkeleton* FPhysicsAssetDataflowState::GetSkeleton() const
{
	return TargetSkeleton.Get();
}

USkeletalMesh* FPhysicsAssetDataflowState::GetMesh() const
{
	return TargetMesh.Get();
}

USkeletalBodySetup* FPhysicsAssetDataflowState::GetBody(FName BoneName) const
{
	if(!HasData())
	{
		return nullptr;
	}

	const FReferenceSkeleton& RefSkel = TargetSkeleton->GetReferenceSkeleton();

	const int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);

	if(BoneIndex == INDEX_NONE)
	{
		return nullptr;
	}

	int32 BodyIndex = BoneIndexToBody[BoneIndex];

	if(BodyIndex == INDEX_NONE)
	{
		return nullptr;
	}

	return Bodies[BodyIndex];
}

void FPhysicsAssetDataflowState::SetBody(FName BoneName, USkeletalBodySetup* InSetup)
{
	if(!HasData())
	{
		return;
	}

	const FReferenceSkeleton& RefSkel = TargetSkeleton->GetReferenceSkeleton();

	const int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);

	if(BoneIndex == INDEX_NONE)
	{
		return;
	}

	int32 BodyIndex = BoneIndexToBody[BoneIndex];

	if(BodyIndex == INDEX_NONE)
	{
		Bodies.Add(InSetup);
		BoneIndexToBody.Modify(BoneIndex, Bodies.Num() - 1);
	}
	else
	{
		Bodies.Modify(BodyIndex, InSetup);
	}
}

USkeletalBodySetup* FPhysicsAssetDataflowState::FindOrCreateBody(FName BoneName, USkeletalBodySetup* TemplateBody)
{
	if(!HasData())
	{
		return nullptr;
	}

	const FReferenceSkeleton& RefSkel = TargetSkeleton->GetReferenceSkeleton();

	const int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);

	if(BoneIndex == INDEX_NONE)
	{
		return nullptr;
	}

	int32 BodyIndex = BoneIndexToBody[BoneIndex];

	if(BodyIndex == INDEX_NONE)
	{
		if(!TemplateBody)
		{
			return nullptr;
		}

		USkeletalBodySetup* NewSetup = CastChecked<USkeletalBodySetup>(StaticDuplicateObject(TemplateBody, GetTransientPackage()));
		NewSetup->BoneName = BoneName;

		Bodies.Add(NewSetup);
		BoneIndexToBody.Modify(BoneIndex, Bodies.Num() - 1);

		return Bodies.Last();
	}

	return Bodies[BodyIndex];
}

USkeletalBodySetup* FPhysicsAssetDataflowState::FindOrCreateBodyChecked(FName BoneName, USkeletalBodySetup* TemplateBody /*= nullptr*/)
{
	USkeletalBodySetup* Body = FindOrCreateBody(BoneName, TemplateBody);
	check(Body);

	return Body;
}

const TCopyOnWriteArray<TObjectPtr<USkeletalBodySetup>>& FPhysicsAssetDataflowState::GetBodies() const
{
	return Bodies;
}

void FPhysicsAssetDataflowState::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TargetSkeleton);
	Collector.AddReferencedObject(TargetMesh);
	AddArrayReferences(Collector, Bodies, Constraints);
}

void FPhysicsAssetDataflowState::DebugLog() const
{
	for(const USkeletalBodySetup* Setup : Bodies)
	{
		if(!Setup)
		{
			continue;
		}

		UE_LOGF(LogTemp, Warning, "Body: %ls", *Setup->GetName());
	}

	for(const UPhysicsConstraintTemplate* Constraint : Constraints)
	{
		if(!Constraint)
		{
			continue;
		}

		UE_LOGF(LogTemp, Warning, "Constraint: %ls", *Constraint->GetName());
	}
}

bool FPhysicsAssetDataflowState::HasData() const
{
	return TargetSkeleton && TargetMesh;
}
