// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "ShapeElemNodes.h"
#include "Dataflow/DataflowAttachment.h"

#include "RigidDataflowNode.generated.h"

/**
 * Intermediate node for rigid asset node implementations encapsulating common functionality
 */
USTRUCT()
struct FRigidDataflowNode : public FDataflowNode
{
	GENERATED_BODY();

public:

	FRigidDataflowNode() = default;

	FRigidDataflowNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
		: FDataflowNode(Param, InGuid)
		, Owner(Param.OwningObject)
	{

	}

	// Get the owning UObject for the node, to store UObjects in the asset safely
	TObjectPtr<UObject> GetOwner() const
	{
		return Owner;
	}

	// Multi-input versions of the pin registration functions
	template<typename... T>
	void AddInputs(T&&... Args)
	{
		(RegisterInputConnection(&Args), ...);
	}

	template<typename... T>
	void AddOutputs(T&&... Args)
	{
		(RegisterOutputConnection(&Args), ...);
	}

	template<typename... T>
	void AddPassthroughs(T&&... Args)
	{
		(RegisterOutputConnection(&Args, &Args), ...);
	}

protected:

	// Grab the context owner and return it as T if possible, otherwise nullptr
	template<typename T>
	TObjectPtr<T> GetContextOwnerAs(UE::Dataflow::FContext& InContext) const
	{
		if(UE::Dataflow::FEngineContext* EngineContext = InContext.AsType<UE::Dataflow::FEngineContext>())
		{
			return Cast<T>(EngineContext->Owner);
		}

		return nullptr;
	}

	TObjectPtr<UObject> GetContextOwner(UE::Dataflow::FContext& InContext) const
	{
		if(UE::Dataflow::FEngineContext* EngineContext = InContext.AsType<UE::Dataflow::FEngineContext>())
		{
			return EngineContext->Owner;
		}

		return nullptr;
	}

	// Get the dataflow attachement if it is avaialble for the current asset
	TObjectPtr<UDataflowAttachment> GetAttachment(UE::Dataflow::FContext& InContext) const
	{
		return GetContextOwnerAs<UDataflowAttachment>(InContext);
	}

	TObjectPtr<UObject> GetAttachmentOwner(UE::Dataflow::FContext& InContext) const
	{
		if(TObjectPtr<UDataflowAttachment> Attachment = GetAttachment(InContext))
		{
			return Attachment->GetOuter();
		}

		return nullptr;
	}

	template<typename T>
	TObjectPtr<T> GetAttachmentOwnerAs(UE::Dataflow::FContext& InContext) const
	{
		if(TObjectPtr<UDataflowAttachment> Attachment = GetAttachment(InContext))
		{
			return Cast<T>(Attachment->GetOuter());
		}

		return nullptr;
	}

private:

	UPROPERTY()
	TObjectPtr<UObject> Owner;

};

struct IMultiPinArrayConnector
{
	virtual ~IMultiPinArrayConnector()
	{
	}

	virtual void InitPins() = 0;
	virtual TArray<UE::Dataflow::FPin> AddPins() = 0;
	virtual bool CanAddPin() const = 0;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const = 0;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) = 0;
	virtual bool CanRemovePin() const = 0;
	virtual void PostSerialize(const FArchive& Ar) = 0;
};

template<typename T>
struct TMultiPinArrayConnector : public IMultiPinArrayConnector
{
	TMultiPinArrayConnector() = delete;
	explicit TMultiPinArrayConnector(FRigidDataflowNode& InNode, TArray<T>& InPinContainer, int32 InMinimumNumPins)
		: Node(InNode)
		, PinContainer(InPinContainer)
		, MinimumNumPins(InMinimumNumPins)
	{
		
	}

	void InitPins() override
	{
		check(PinContainer.Num() == 0);

		for(int32 Index = 0; Index < MinimumNumPins; ++Index)
		{
			AddPins();
		}
	}

	TArray<UE::Dataflow::FPin> AddPins() override
	{
		const int32 Index = PinContainer.AddDefaulted();
		UE::Dataflow::TConnectionReference<UE::Chaos::RigidAsset::FSimpleGeometry> Ref{ &PinContainer[Index], Index, &PinContainer };
		const FDataflowInput& Input = Node.RegisterInputArrayConnection(Ref);

		return
		{
			{
				.Direction = UE::Dataflow::FPin::EDirection::INPUT,
				.Type = Input.GetType(),
				.Name = Input.GetName()
			}
		};
	}


	bool CanAddPin() const override
	{
		return true;
	}


	TArray<UE::Dataflow::FPin> GetPinsToRemove() const override
	{
		if(PinContainer.Num() == 0)
		{
			return {};
		}

		const int32 Index = PinContainer.Num() - 1;
		UE::Dataflow::TConnectionReference<UE::Chaos::RigidAsset::FSimpleGeometry> Ref{ &PinContainer[Index], Index, &PinContainer };

		if(const FDataflowInput* Input = Node.FindInput(Ref))
		{
			return
			{
				{
					.Direction = UE::Dataflow::FPin::EDirection::INPUT,
					.Type = Input->GetType(),
					.Name = Input->GetName()
				}
			};
		}

		return {};
	}


	void OnPinRemoved(const UE::Dataflow::FPin& Pin) override
	{
		PinContainer.Pop();
	}

	bool CanRemovePin() const override
	{
		return PinContainer.Num() > MinimumNumPins;
	}
	
	void PostSerialize(const FArchive& Ar) override
	{
		// Restore pins
		if(Ar.IsLoading())
		{
			for(int32 Index = 0; Index < PinContainer.Num(); ++Index)
			{
				UE::Dataflow::TConnectionReference<UE::Chaos::RigidAsset::FSimpleGeometry> Ref{ &PinContainer[Index], Index, &PinContainer };
				Node.FindOrRegisterInputArrayConnection(Ref);
			}

			if(Ar.IsTransacting())
			{
				int32 OriginalNum = Node.GetNumInputs();
				const int32 ExpectedNum = PinContainer.Num();

				if(OriginalNum > ExpectedNum)
				{
					PinContainer.SetNum(OriginalNum);

					while(OriginalNum-- > ExpectedNum)
					{
						const int32 Index = PinContainer.Num() - 1;
						UE::Dataflow::TConnectionReference<UE::Chaos::RigidAsset::FSimpleGeometry> Ref{ &PinContainer[Index], Index, &PinContainer };
						Node.UnregisterInputConnection(Ref);

						PinContainer.Pop(EAllowShrinking::No);
					}

					check(PinContainer.Num() == ExpectedNum);
				}
			}
		}
	}

private:

	FRigidDataflowNode& Node;
	TArray<T>& PinContainer;
	int32 MinimumNumPins;
};

USTRUCT()
struct FRigidDataflowMultiInputNode : public FRigidDataflowNode
{
	GENERATED_BODY()

	FRigidDataflowMultiInputNode() = default;

	template<typename T>
	explicit FRigidDataflowMultiInputNode(TArray<T>& InPinContainer, const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
		: FRigidDataflowNode(InParam, InGuid)
	{
		Connector = MakeUnique<TMultiPinArrayConnector<T>>(*this, InPinContainer, 0);
	}

	void InitPins()
	{
		if(Connector)
		{
			Connector->InitPins();
		}
	}

	TArray<UE::Dataflow::FPin> AddPins() override
	{
		if(Connector)
		{
			return Connector->AddPins();
		}

		return Super::AddPins();
	}

	bool CanAddPin() const override
	{
		if(Connector)
		{
			return Connector->CanAddPin();
		}

		return Super::CanAddPin();
	}


	TArray<UE::Dataflow::FPin> GetPinsToRemove() const override
	{
		if(Connector)
		{
			return Connector->GetPinsToRemove();
		}

		return Super::GetPinsToRemove();
	}


	void OnPinRemoved(const UE::Dataflow::FPin& Pin) override
	{
		if(Connector)
		{
			Connector->OnPinRemoved(Pin);
			return;
		}

		Super::OnPinRemoved(Pin);
	}

	bool CanRemovePin() const override
	{
		if(Connector)
		{
			return Connector->CanRemovePin();
		}

		return Super::CanRemovePin();
	}

	void PostSerialize(const FArchive& Ar) override
	{
		if(Connector)
		{
			Connector->PostSerialize(Ar);
		}

		Super::PostSerialize(Ar);
	}

private:
	TUniquePtr<IMultiPinArrayConnector> Connector = nullptr;
};

template<>
struct TStructOpsTypeTraits<FRigidDataflowMultiInputNode> : public TStructOpsTypeTraitsBase2<FRigidDataflowMultiInputNode>
{
	enum
	{
		WithCopy = false
	};
};