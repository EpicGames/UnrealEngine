// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShapeElemNodes.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "PhysicsEngine/ShapeElem.h"
#include "PhysicsEngine/ConvexElem.h"

namespace UE::Dataflow
{
	void RegisterShapeNodes()
	{
		using namespace UE::Chaos::RigidAsset;

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeBoxElemDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSphereElemDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCapsuleElemDataflowNode);
	}
}

namespace UE::Chaos::RigidAsset
{
	void FMakeBoxElemDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
	{
		FSimpleGeometry NewGeom;
		NewGeom.GeometryElement = MakeShared<FKBoxElem>(Box);

		SetValue(Context, MoveTemp(NewGeom), &Geometry);
	}

	void FMakeBoxElemDataflowNode::Register()
	{
		RegisterOutputConnection(&Geometry);
	}

	// Helper to avoid handling every access in a switch statement
	// TODO move to Engine
	template<typename Callable>
	auto VisitAsConcreteElement(FKShapeElem* InElem, Callable&& InCallable)
	{
		switch(InElem->GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			return InCallable(InElem->GetShapeCheck<FKSphereElem>());
		case EAggCollisionShape::Box:
			return InCallable(InElem->GetShapeCheck<FKBoxElem>());
		case EAggCollisionShape::Sphyl:
			return InCallable(InElem->GetShapeCheck<FKSphylElem>());
		case EAggCollisionShape::Convex:
			return InCallable(InElem->GetShapeCheck<FKConvexElem>());
		default:
			UE_LOGF(LogTemp, Warning, "Unsupported rigid shape type");
		}

		return InCallable(static_cast<FKShapeElem*>(nullptr));
	}

	FSimpleGeometry FSimpleGeometry::Duplicate()
	{
		TSharedPtr<FKShapeElem> NewElem;
		
		VisitAsConcreteElement(GeometryElement.Get(), [&NewElem] <typename ElemType> (ElemType* Elem)
			{
				check(Elem);

				NewElem = MakeShared<ElemType>(*Elem);
			});

		return { LocalTransform, NewElem };
	}

	void FMakeSphereElemDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
	{
		FSimpleGeometry NewGeom;
		NewGeom.GeometryElement = MakeShared<FKSphereElem>(Sphere);

		SetValue(Context, MoveTemp(NewGeom), &Geometry);
	}

	void FMakeSphereElemDataflowNode::Register()
	{
		RegisterOutputConnection(&Geometry);
	}

	void FMakeCapsuleElemDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
	{
		FSimpleGeometry NewGeom;
		NewGeom.GeometryElement = MakeShared<FKSphylElem>(Capsule);

		SetValue(Context, MoveTemp(NewGeom), &Geometry);
	}

	void FMakeCapsuleElemDataflowNode::Register()
	{
		RegisterOutputConnection(&Geometry);
	}
}
