// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "Dataflow/DataflowNode.h"

#include "ShapeElemNodes.generated.h"

namespace UE::Dataflow
{
	void RegisterShapeNodes();
}

namespace UE::Chaos::RigidAsset
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Generic container for any geometry element while referenced in a dataflow graph
	 */
	USTRUCT()
	struct FSimpleGeometry
	{
		GENERATED_BODY();

		FSimpleGeometry() = default;

		FSimpleGeometry(TSharedPtr<FKShapeElem> InElem)
			: GeometryElement(InElem)
		{
		}

		FSimpleGeometry(const FTransform InLocalTransform, TSharedPtr<FKShapeElem> InElem)
			: LocalTransform(InLocalTransform)
			, GeometryElement(InElem)
		{
		}

		// Duplicate this geometry handling the geometry type correctly
		FSimpleGeometry Duplicate();

		FTransform LocalTransform = FTransform::Identity;
		TSharedPtr<FKShapeElem> GeometryElement = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Make single-element nodes.
	 */
	USTRUCT()
	struct FMakeBoxElemDataflowNode : public FDataflowNode
	{
		GENERATED_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FMakeBoxElemDataflowNode, "MakeBox", "Rigid Shapes", "")
	
	public:
	
		FMakeBoxElemDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
			: FDataflowNode(InParam, InGuid)
		{
			Register();
		}

	private:

		// Input box data
		UPROPERTY(EditAnywhere, Category=Settings)
		FKBoxElem Box;
	
		// Produced geometry
		UPROPERTY(meta = (DataflowOutput))
		FSimpleGeometry Geometry;

		void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
		void Register();
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	USTRUCT()
	struct FMakeSphereElemDataflowNode : public FDataflowNode
	{
		GENERATED_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSphereElemDataflowNode, "MakeSphere", "Rigid Shapes", "")
	
	public:
	
		FMakeSphereElemDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
			: FDataflowNode(InParam, InGuid)
		{
			Register();
		}

	private:

		// Input Sphere data
		UPROPERTY(EditAnywhere, Category=Settings)
		FKSphereElem Sphere;
	
		// Produced geometry
		UPROPERTY(meta = (DataflowOutput))
		FSimpleGeometry Geometry;

		void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
		void Register();
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	USTRUCT()
	struct FMakeCapsuleElemDataflowNode : public FDataflowNode
	{
		GENERATED_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCapsuleElemDataflowNode, "MakeCapsule", "Rigid Shapes", "")
	
	public:
	
		FMakeCapsuleElemDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
			: FDataflowNode(InParam, InGuid)
		{
			Register();
		}

	private:

		// Input capsule data
		UPROPERTY(EditAnywhere, Category=Settings)
		FKSphylElem Capsule;
	
		// Produced geometry
		UPROPERTY(meta = (DataflowOutput))
		FSimpleGeometry Geometry;

		void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
		void Register();
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
}