// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Containers/Array.h"

#include "BoneSelection.generated.h"

class USkeleton;
class USkeletalMesh;

/**
 * Represents a single bone in a selection
 */
USTRUCT()
struct FRigidAssetBoneInfo
{
	GENERATED_BODY()

	FRigidAssetBoneInfo() = default;
	FRigidAssetBoneInfo(FName InName, int32 InIndex, int32 InDepth);

	// Bone name in the skeleton
	UPROPERTY()
	FName Name = NAME_None;

	// Raw bone index in the skeleton
	UPROPERTY()
	int32 Index = INDEX_NONE;

	// Depth of this bone in the hierarchy (distance from root)
	UPROPERTY()
	int32 Depth = INDEX_NONE;

	// List of child bones merged into this one
	UPROPERTY()
	TArray<int32> MergedChildren;

	bool operator ==(const FRigidAssetBoneInfo& Other) const = default;
	bool operator <(const FRigidAssetBoneInfo& Other) const;
};

/**
 * Represents a selection set of bones within a skeleton
 */
USTRUCT()
struct FRigidAssetBoneSelection
{
	GENERATED_BODY()

	// Individual bones in the selection
	UPROPERTY()
	TArray<FRigidAssetBoneInfo> SelectedBones;

	// Skeleton that the bones reference
	UPROPERTY()
	TObjectPtr<USkeleton> Skeleton;

	// Mesh that the bones reference
	UPROPERTY()
	TObjectPtr<USkeletalMesh> Mesh;

	// Contains for index and name, cannot check the skeleton matches, so care should be taken
	// that the search items here are representative of the mesh and skeleton bound to the selection
	bool ContainsIndex(int32 InBoneIndex) const;
	bool Contains(FName InBoneName) const;

	// Sorts bones so parents are before children
	void SortBones();
};
