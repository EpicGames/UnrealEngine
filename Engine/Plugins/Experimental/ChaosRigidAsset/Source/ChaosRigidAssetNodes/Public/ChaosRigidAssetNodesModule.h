// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FChaosRigidAssetNodesModule : public IModuleInterface
{
public:

	CHAOSRIGIDASSETNODES_API virtual void StartupModule() override;
	CHAOSRIGIDASSETNODES_API virtual void ShutdownModule() override;

private:
};