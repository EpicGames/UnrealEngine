// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Concepts/ConvertibleTo.h"
#include "Misc/ScopeRWLock.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectPtr.h"
#include "Dataflow/DataflowContextCache.h"

class UObject;
class UClass;
class UScriptStruct;

/**
 * Concept that detects a ptr to UObject type, requiring:
 *   - T is a ptr type
 *   - T is convertible UObject*
 *   - T Has a method GetClass the returns a UClass
 */
template<typename T>
concept CIsUObjectType = requires (T Object)
{
	requires UE::CConvertibleTo<T, UObject*>;
	{ Object->GetClass() } -> UE::CConvertibleTo<UClass*>;
};

/**
 * Concept that detects a USTRUCT() type, requiring:
 *   - T has a static method StaticStruct that returns a UScriptStruct*
 */
template<typename T>
concept CIsUStructType = requires (T Object)
{
	{ T::StaticStruct() } -> UE::CConvertibleTo<UScriptStruct*>;
};

/**
 * Wrapper for an array that implements copy-on-write for the array.
 * If there are N readers but no writers there will be a single copy of the array
 * If a codepath writes to the array and they aren't the only referencer, the array will be duplicated
 * This means each modification will not implicitly duplicate - only if there are multiple referencers
 * otherwise, we modify in place.
 */
template<typename T>
class TCopyOnWriteArray
{
public:
	using InnerArrayType = TArray<T>;
	using AllocatorType = InnerArrayType::AllocatorType;
	using SizeType = InnerArrayType::SizeType;

	TCopyOnWriteArray()
	{

	}

	TCopyOnWriteArray(const TCopyOnWriteArray& Other)
	{
		State = Other.State;
	}

	TCopyOnWriteArray(TCopyOnWriteArray&& Other)
	{
		State = MoveTemp(Other.State);
	}

	TCopyOnWriteArray& operator=(const TCopyOnWriteArray& Other)
	{
		State = Other.State;
		return *this;
	}

	TCopyOnWriteArray& operator=(TCopyOnWriteArray&& Other)
	{
		State = MoveTemp(Other.State);
		return *this;
	}

	const T& operator [](int32 Index) const
	{
		checkf(State, TEXT("Array index out of bounds: %d into an array of size 0"), Index);
		return State->Arr[Index];
	}

	TArray<T>::RangedForConstIteratorType begin() const
	{
		if(!State)
		{
			static const TArray<T> EmptyArray;
			return EmptyArray.begin();
		}

		const TArray<T>& Arr = State->Arr;
		return Arr.begin();
	}

	TArray<T>::RangedForConstIteratorType end() const
	{
		if(!State)
		{
			static const TArray<T> EmptyArray;
			return EmptyArray.end();
		}

		const TArray<T>& Arr = State->Arr;
		return Arr.end();
	}

	template<typename... Args>
	int32 Emplace(Args&&... InArgs)
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr.Emplace(Forward<Args>(InArgs)...);

		return State->Arr.Num();
	}

	int32 Add(const T& Value)
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr.Add(Value);

		return State->Arr.Num();
	}

	int32 Add(T&& Value)
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr.Add(Forward<T>(Value));

		return State->Arr.Num();
	}

	void Modify(int32 AtIndex, const T& Value)
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr[AtIndex] = Value;
	}

	void Modify(int32 AtIndex, T&& Value)
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr[AtIndex] = MoveTemp(Value);
	}

	void Append(const TArray<T>& Other)
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr.Append(Other);
	}

	void SetNum(SizeType NewNum, EAllowShrinking AllowShrink = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr.SetNum(NewNum, AllowShrink);
	}

	void SetNumZeroed(SizeType NewNum, EAllowShrinking AllowShrink = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		FWriteScopeLock ScopeLock(Lock);

		CheckCopyOnWrite();
		State->Arr.SetNumZeroed(NewNum, AllowShrink);
	}

	SizeType Num() const
	{
		return State ? State->Arr.Num() : 0;
	}

	const T& Last(SizeType IndexFromEnd = 0) const
	{
		checkf(State, TEXT("Array index out of bounds: Last() accessing empty array"));
		return State->Arr.Last(IndexFromEnd);
	}

	// If the array contains UObjects, we need to emit references to them directly
	void AddReferencedObjects(FReferenceCollector& Collector) const
		requires CIsUObjectType<T>
	{
		if(!State)
		{
			return;
		}

		for(const T& Item : State->Arr)
		{
			TObjectPtr<UObject> AsObject = Item;
			Collector.AddReferencedObject(AsObject);
			AsObject->AddReferencedObjects(AsObject, Collector);
		}
	}

	// If the array contains USTRUCT types, they might reference UObject types and
	// need to emit references. Defer to the struct here to add required references
	void AddReferencedObjects(FReferenceCollector& Collector) const
		requires CIsUStructType<T>
	{
		if(!State)
		{
			return;
		}

		UScriptStruct* Struct = T::StaticStruct();

		for(const T& Item : State->Arr)
		{
			Struct->GetCppStructOps()->AddStructReferencedObjects()((void*)&Item, Collector);
		}
	}

	// Base case for storage types that don't emit references
	void AddReferencedObjects(FReferenceCollector& Collector) const
	{

	}

	// Copy the internal state into a new TArray
	TArray<T> ToArray() const
	{
		if(!State)
		{
			return {};
		}

		TArray<T> NewArray;
		NewArray.Reserve(State->Arr.Num());
		NewArray.Append(State->Arr);

		return MoveTemp(NewArray);
	}

private:

	void CheckCopyOnWrite()
	{
		// If we have no state, or there are more than one reader, duplicate before writing
		if(!State || State->GetRefCount() > 1)
		{
			Duplicate();
		}
	}

	void Duplicate()
	{
		TRefCountPtr<FState> Curr = State;
		TRefCountPtr<FState> New = new FState;

		// Duplicate if there's a previous state
		if(Curr)
		{
			// It's safe to iterate and append the old array here. If we're duplicating then all writers will duplicate.
			// The current state is logically const at this point in time.
			if constexpr(UE::Dataflow::template TIsUObjectPtrElement<T>::Value)
			{
				// UObjects need to run through StaticDuplicateObject rather than just be copied.
				New->Arr.Reserve(Curr->Arr.Num());

				for(const T& From : Curr->Arr)
				{
					New->Arr.Add(Cast<typename TRemoveObjectPointer<T>::Type>(StaticDuplicateObject(From, GetTransientPackage())));
				}
			}
			else
			{
				New->Arr.Append(Curr->Arr);
			}
		}

		State = New;
	}

	struct FState : public UE::Private::FQueryableRefCountedObject
	{
		TArray<T> Arr;
	};

	TRefCountPtr<FState> State = nullptr;
	FRWLock Lock;
};

// Variadic helper to call AddReferencedObjects on multiple arrays
template<typename... T>
void AddArrayReferences(FReferenceCollector& Collector, const T&... Args)
{
	(Args.AddReferencedObjects(Collector), ...);
}